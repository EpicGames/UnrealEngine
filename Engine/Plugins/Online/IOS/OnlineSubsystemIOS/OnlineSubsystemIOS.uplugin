{
	"FileVersion" : 3,
	
	"FriendlyName" : "Online Subsystem iOS",
	"Version" : 1,
	"VersionName" : "1.0",
	"Description" : "Access to iOS platform",
	"Category" : "Online Platform",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"EnabledByDefault" : true,
	"SupportedTargetPlatforms": [
		"IOS",
		"TVOS",
	],
    "Modules": [
        {
            "Name": "OnlineSubsystemIOS",
            "Type": "Runtime",
            "LoadingPhase": "Default",
            "PlatformAllowList" :
			[
                "IOS",
                "TVOS"
			]
        }
    ],
	"LocalizationTargets": [
		{
			"Name": "OnlineSubsystemIOS",
			"LoadingPolicy": "Always"
		}
	],
	"Plugins": [
		{
			"Name": "OnlineSubsystem",
			"Enabled": true
		}
	]

}