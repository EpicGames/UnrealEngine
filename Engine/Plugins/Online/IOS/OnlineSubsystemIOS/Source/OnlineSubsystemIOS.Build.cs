// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class OnlineSubsystemIOS : ModuleRules
{
    public OnlineSubsystemIOS(ReadOnlyTargetRules Target) : base(Target)
    {
		bRequiresPlatformSDK = true;

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		PrivateIncludePaths.AddRange( 
            new string[] {
                "Private",
                });

        PublicIncludePaths.AddRange(
            new string[] {
                "Runtime/IOS/IOSPlatformFeatures/Public"
                });

        PublicDefinitions.Add("ONLINESUBSYSTEMIOS_PACKAGE=1");

		PrivateDependencyModuleNames.AddRange(
            new string[] { 
                "Core", 
                "CoreUObject", 
                "Engine", 
                "HTTP",
                "IOSPlatformFeatures",
				"OnlineSubsystem", 
                "Sockets",
            }
            );

		PublicWeakFrameworks.Add("CloudKit");
		SwiftInteropHeader = System.IO.Path.Combine(ModuleDirectory, "Private/StoreKitSwiftInterop.h");
		
		if (Target.Platform == UnrealTargetPlatform.IOS)
		{
			PublicWeakFrameworks.Add("MultipeerConnectivity");
			PrivateDependencyModuleNames.Add("MarketplaceKit");
		}
	}
}
