// Copyright Epic Games, Inc. All Rights Reserved.

// Module includes
#include "OnlineAchievementsInterfaceIOS.h"

#include "Apple/AppleStringUtils.h"

#include "OnlineSubsystem.h"

// Game Center includes
#include <GameKit/GKAchievement.h>
#include <GameKit/GKAchievementDescription.h>

#include "IOS/IOSAsyncTask.h"

FOnlineAchievementsIOS::FOnlineAchievementsIOS(class FOnlineSubsystemIOS* InSubsystem)
{
	UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("FOnlineSubsystemIOS::FOnlineAchievementsIOS()"));
	IOSSubsystem = InSubsystem;
}

void FOnlineAchievementsIOS::QueryAchievements(const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate)
{
	// Make a copy of the delegate so it won't be a reference inside the FAppleAsyncTask
	FOnQueryAchievementsCompleteDelegate CopiedDelegate = Delegate;

	dispatch_async(dispatch_get_main_queue(), ^
	{
	[GKAchievement loadAchievementsWithCompletionHandler: ^(NSArray *loadedAchievements, NSError *error)
		{
			bool bSuccess = (error == NULL);
			UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("FOnlineSubsystemIOS::loadAchievementsWithCompletionHandler - %d"), bSuccess);

			if (bSuccess)
			{
				// Write the achievements back to our Achievements array
				[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
				{
					UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("Loaded %lld achievements"), [loadedAchievements count]);
					Achievements.Empty();

					for( int32 AchievementIdx = 0; AchievementIdx < [loadedAchievements count]; AchievementIdx++ )
					{
						GKAchievement* LoadedAchievement = [loadedAchievements objectAtIndex : AchievementIdx];
						UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("Loaded achievement: %s"), *FString(LoadedAchievement.identifier));

						Achievements.AddZeroed();
						FOnlineAchievement& OnlineAchievement = Achievements[AchievementIdx];
						OnlineAchievement.Id = LoadedAchievement.identifier;
						OnlineAchievement.Progress = LoadedAchievement.percentComplete;
					}

					CopiedDelegate.ExecuteIfBound(PlayerId, true);
					return true;
				}];
			}
			else
			{
				UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("Failed to load achievements with error [%lld]"), [error code]);

				// Report back to the game thread this failed.
				[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
				{
					CopiedDelegate.ExecuteIfBound(PlayerId, false);
					return true;
				}];
			}
		}
	];
	});
}

void FOnlineAchievementsIOS::QueryAchievementDescriptions(const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate)
{
	FOnQueryAchievementsCompleteDelegate CopiedDelegate = Delegate;

	dispatch_async(dispatch_get_main_queue(), ^
	{
		[GKAchievementDescription loadAchievementDescriptionsWithCompletionHandler : ^ (NSArray *descriptions, NSError *error)
		{
			bool bSuccess = (error == NULL);
			UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("FOnlineSubsystemIOS::loadAchievementDescriptionsWithCompletionHandler - %d"), bSuccess);

			if (bSuccess)
			{				
				// Report back to the game thread whether this succeeded.
				[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
				{
					AchievementDescriptions.Empty();

					for (GKAchievementDescription* desc in descriptions)
					{
						FString Id(desc.identifier);

						FOnlineAchievementDesc OnlineAchievementDesc;
						OnlineAchievementDesc.Title = FText::FromString(desc.title);
						OnlineAchievementDesc.LockedDesc = FText::FromString(desc.unachievedDescription);
						OnlineAchievementDesc.UnlockedDesc = FText::FromString(desc.achievedDescription);
						OnlineAchievementDesc.bIsHidden = desc.hidden;

						UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("============================================"));
						UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("Loaded achievement id: %s"), *Id);
						UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("Loaded achievement title: %s"), *OnlineAchievementDesc.Title.ToString());
						UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("Loaded achievement locked desc: %s"), *OnlineAchievementDesc.LockedDesc.ToString());
						UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("Loaded achievement unlocked desc: %s"), *OnlineAchievementDesc.UnlockedDesc.ToString());
						UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("Loaded achievement hidden: %d"), OnlineAchievementDesc.bIsHidden);
						UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("============================================"));

						AchievementDescriptions.Add(Id, OnlineAchievementDesc);
					}

					CopiedDelegate.ExecuteIfBound(PlayerId, true);
					return true;
				}];
			}
			else
			{
				UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("Failed to load achievement descriptions with error [%lld]"), [error code]);

				// Report the failure back to the game thread
				[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
				{
					CopiedDelegate.ExecuteIfBound(PlayerId, false);
					return true;
				}];
			}
		}];
	});
}


void FOnlineAchievementsIOS::WriteAchievements(const FUniqueNetId& PlayerId, FOnlineAchievementsWriteRef& InWriteObject, const FOnAchievementsWrittenDelegate& Delegate)
{
	// Make a copy of the delegate so it won't be a reference inside the FAppleAsyncTask
	FOnAchievementsWrittenDelegate CopiedDelegate = Delegate;

	// Hold a reference to the write object for the completion handler to write to.
	FOnlineAchievementsWriteRef WriteObject = InWriteObject;
	WriteObject->WriteState = EOnlineAsyncTaskState::InProgress;

	NSMutableArray* UnreportedAchievements = [NSMutableArray arrayWithCapacity:WriteObject->Properties.Num()];
	[UnreportedAchievements retain];

	int32 StatIdx = 0;
	for (FStatPropertyArray::TConstIterator It(WriteObject->Properties); It; ++It)
	{
		// Access the stat and the value.
		const FVariantData& Stat = It.Value();

		// Create an achievement object which should be reported to the server.
		const FString AchievementName(It.Key());
		NSString* AchievementID = [NSString stringWithFString:AchievementName];
		
		GKAchievement* Achievement = [[[GKAchievement alloc] initWithIdentifier:AchievementID] autorelease];
		Achievement.showsCompletionBanner = YES;

		// Setup the percentage complete with the value we are writing from the variant type
		switch (Stat.GetType())
		{
			case EOnlineKeyValuePairDataType::Int32:
			{
				int32 Value;
				Stat.GetValue(Value);
				Achievement.percentComplete = (float)Value;

				break;
			}
			case EOnlineKeyValuePairDataType::Float:
			{
				float Value;
				Stat.GetValue(Value);
				Achievement.percentComplete = Value;

				break;
			}
			default:
			{
				UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("FOnlineSubsystemIOS Trying to write an achievement with incompatible format. Not a float or int"));
				break;
			}
		}

		[UnreportedAchievements addObject:Achievement];
	}

	// flush the achievements to the server
	if ([UnreportedAchievements count] > 0)
	{
		dispatch_async(dispatch_get_main_queue(), ^
		{
			[GKAchievement reportAchievements : UnreportedAchievements withCompletionHandler : ^ (NSError* Error)
			{
				if (Error == nil)
				{
					for (GKAchievement* achievement in UnreportedAchievements)
					{
						FString AchievementId(achievement.identifier);
						UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("Successfully reported achievement: %s, isCompleted:%d"), *AchievementId, [achievement isCompleted] ? 1 : 0);

						// Report any completed achievements to the game thread
						if ([achievement isCompleted])
						{
							[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
							{
								TriggerOnAchievementUnlockedDelegates(PlayerId, AchievementId);
								return true;
							}];
						}
					}
				}
				else
				{
					UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("Failed to report achievements with error[%lld]"), [Error code]);
				}

				// Release any handle of the achievements back to the IOS gc
				[UnreportedAchievements release];

				// Report whether this succeeded or not back to whoever is listening.
				WriteObject->WriteState = (Error == nil) ? EOnlineAsyncTaskState::Done : EOnlineAsyncTaskState::Failed;

				// Report back to the game thread whether this succeeded.
				[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
				{
					CopiedDelegate.ExecuteIfBound(PlayerId, Error == nil);
					return true;
				}];
			}
		];
		});
	}
	else
	{
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("No achievements were written to be flushed"));
		WriteObject->WriteState = EOnlineAsyncTaskState::Failed;
		CopiedDelegate.ExecuteIfBound(PlayerId, false);
	}
}


EOnlineCachedResult::Type FOnlineAchievementsIOS::GetCachedAchievement(const FUniqueNetId& PlayerId, const FString& AchievementId, FOnlineAchievement& OutAchievement)
{
	for (int32 AchievementIdx = 0; AchievementIdx < Achievements.Num(); AchievementIdx++)
	{
		if (Achievements[AchievementIdx].Id == AchievementId)
		{
			OutAchievement = Achievements[AchievementIdx];
			return EOnlineCachedResult::Success;
		}
	}

	return EOnlineCachedResult::NotFound;
}


EOnlineCachedResult::Type FOnlineAchievementsIOS::GetCachedAchievements(const FUniqueNetId& PlayerId, TArray<FOnlineAchievement>& OutAchievements)
{
	// look up achievements for player
	OutAchievements = Achievements;

	// did we have them cached?
	return (OutAchievements.Num() > 0) ? EOnlineCachedResult::Success : EOnlineCachedResult::NotFound;
}

EOnlineCachedResult::Type FOnlineAchievementsIOS::GetCachedAchievementDescription(const FString& AchievementId, FOnlineAchievementDesc& OutAchievementDesc)
{
	if (FOnlineAchievementDesc* FoundDesc = AchievementDescriptions.Find(AchievementId))
	{
		OutAchievementDesc = *FoundDesc;
		return EOnlineCachedResult::Success;
	}
	else
	{
		return EOnlineCachedResult::NotFound;
	}
}

#if !UE_BUILD_SHIPPING
bool FOnlineAchievementsIOS::ResetAchievements( const FUniqueNetId& PlayerId )
{
	dispatch_async(dispatch_get_main_queue(), ^
	{
		[GKAchievement resetAchievementsWithCompletionHandler : ^(NSError *error)
		{
			bool bSuccess = error == NULL;
			UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("FOnlineAchievementsIOS::ResetAchievements - %d"), bSuccess ? 1 : 0);

			if (bSuccess)
			{
				// Wipe out the achievement descriptions back on the game thread.
				[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
				{
					Achievements.Empty();
					AchievementDescriptions.Empty();
					return true;
				}];
			}
		}
		];
	});

	return true;
};
#endif // !UE_BUILD_SHIPPING
