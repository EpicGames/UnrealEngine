// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineAchievementsInterface.h"

/**
 *	IOnlineAchievements - Interface class for acheivements
 */
class FOnlineAchievementsIOS : public IOnlineAchievements
{
private:

	/** Reference to the main GameCenter subsystem */
	class FOnlineSubsystemIOS* IOSSubsystem;

	/** hide the default constructor, we need a reference to our OSS */
	FOnlineAchievementsIOS() {};

	// IOS only supports loading achievements for local player. This is where they are cached.
	TArray< FOnlineAchievement > Achievements;

	// Cached achievement descriptions for an Id
	TMap< FString, FOnlineAchievementDesc > AchievementDescriptions;


public:

	//~ Begin IOnlineAchievements Interface
	virtual void WriteAchievements(const FUniqueNetId& PlayerId, FOnlineAchievementsWriteRef& WriteObject, const FOnAchievementsWrittenDelegate& Delegate = FOnAchievementsWrittenDelegate()) override;
	virtual void QueryAchievements(const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate = FOnQueryAchievementsCompleteDelegate()) override;
	virtual void QueryAchievementDescriptions( const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate = FOnQueryAchievementsCompleteDelegate() ) override;
	virtual EOnlineCachedResult::Type GetCachedAchievement(const FUniqueNetId& PlayerId, const FString& AchievementId, FOnlineAchievement& OutAchievement) override;
	virtual EOnlineCachedResult::Type GetCachedAchievements(const FUniqueNetId& PlayerId, TArray<FOnlineAchievement> & OutAchievements) override;
	virtual EOnlineCachedResult::Type GetCachedAchievementDescription(const FString& AchievementId, FOnlineAchievementDesc& OutAchievementDesc) override;
#if !UE_BUILD_SHIPPING
	virtual bool ResetAchievements( const FUniqueNetId& PlayerId ) override;
#endif // !UE_BUILD_SHIPPING
	//~ End IOnlineAchievements Interface


	/**
	 * Constructor
	 *
	 * @param InSubsystem - A reference to the owning subsystem
	 */
	FOnlineAchievementsIOS( class FOnlineSubsystemIOS* InSubsystem );

	
	/**
	 * Default destructor
	 */
	virtual ~FOnlineAchievementsIOS() {}
};

typedef TSharedPtr<FOnlineAchievementsIOS, ESPMode::ThreadSafe> FOnlineAchievementsIOSPtr;
