// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineAppStoreUtils.h"
#include "OnlineSubsystem.h"

#include "HAL/PlatformProcess.h"
#include "IOS/IOSAsyncTask.h"

#import <Foundation/Foundation.h>

@implementation FAppStoreUtils

- (id)init
{
	self = [super init];
	return self;
}

-(void)queryAppBundleId: (FOnQueryAppBundleIdResponse) completionDelegate
{
	NSString* BundleId = FPlatformProcess::GetGameBundleId().GetNSString();

	CFLocaleRef Locale = CFLocaleCopyCurrent();
	NSString* CountryCodeStr = (NSString*)(CFStringRef)CFLocaleGetValue(Locale, kCFLocaleCountryCode);

	// Retrieve the bundle metadata for the given id and the given region (ISO-2A country code)
	NSString* AppStoreURL = [NSString stringWithFormat:@"http://itunes.apple.com/lookup?bundleId=%@&country=%@", BundleId, CountryCodeStr];
	NSURL* Url = [NSURL URLWithString:AppStoreURL];
	UE_LOG_ONLINE(Verbose, TEXT("Contacting %s for app store metadata"), *FString(AppStoreURL));

	NSURLSession* DefaultSession = [NSURLSession sharedSession];
	NSURLSessionDataTask* DataTask = [DefaultSession dataTaskWithURL: Url completionHandler:^(NSData* Data, NSURLResponse* Response, NSError* Error)
	{
	    if (Error == nil)
		{
		    NSDictionary* Lookup = [NSJSONSerialization JSONObjectWithData:Data options:0 error:nil];
			completionDelegate.ExecuteIfBound(Lookup);
		}
		else
		{
			completionDelegate.ExecuteIfBound(nil);
		}
	}];
	
	[DataTask resume];
}

@end
