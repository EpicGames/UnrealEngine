// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemTypes.h"

#import <Foundation/Foundation.h> 

/**
 * Delegate fires when a query for the app bundle id metadata completes
 *
 * @param ResponseDict dictionary containing the metadata response
 */
DECLARE_DELEGATE_OneParam(FOnQueryAppBundleIdResponse, NSDictionary* /*ResponseDict*/);

/** 
 * Helper class for querying app store metadata
 */
@interface FAppStoreUtils : NSObject
{
};


/** Retrieve app store json data */
-(void)queryAppBundleId: (FOnQueryAppBundleIdResponse) completionDelegate;

@end
