// Copyright Epic Games, Inc. All Rights Reserved.

// Module includes
#include "OnlineExternalUIInterfaceIOS.h"
#include "OnlineIdentityInterfaceIOS.h"
#include "OnlineSubsystemIOS.h"

FOnlineExternalUIIOS::FOnlineExternalUIIOS(FOnlineSubsystemIOS* InSubsystem)
	: Subsystem(InSubsystem)
{
	check(InSubsystem != nullptr);
}

bool FOnlineExternalUIIOS::ShowLoginUI(const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate)
{
	FOnlineIdentityIOS* IdentityInterface = static_cast<FOnlineIdentityIOS*>(Subsystem->GetIdentityInterface().Get());
	
	check(IdentityInterface != nullptr);
	
	if (FUniqueNetIdPtr UniqueNetId = IdentityInterface->GetUniquePlayerId(ControllerIndex))
	{
		Delegate.ExecuteIfBound(UniqueNetId, ControllerIndex, FOnlineError::Success());
		return true;
	}
	
	// Not authenticated, set a handler
	
	// Copy the delegate so that the block can still access it when it runs.
	CopiedDelegate = Delegate;
	
	// add a Login Complete delegat to the Identity Interface and attempt to Login
	CompleteDelegate = IdentityInterface->AddOnLoginCompleteDelegate_Handle(ControllerIndex, FOnLoginCompleteDelegate::CreateRaw(this, &FOnlineExternalUIIOS::OnLoginComplete));
	IdentityInterface->Login(ControllerIndex, FOnlineAccountCredentials());
	
	return true;
}

bool FOnlineExternalUIIOS::ShowFriendsUI(int32 LocalUserNum)
{
	return false;
}

bool FOnlineExternalUIIOS::ShowInviteUI(int32 LocalUserNum, FName SessionName)
{
	return false;
}

bool FOnlineExternalUIIOS::ShowAchievementsUI(int32 LocalUserNum) 
{
	// Will always show the achievements UI for the current local signed-in user
	extern CORE_API void IOSShowAchievementsUI();
	IOSShowAchievementsUI();
	return true;
}

bool FOnlineExternalUIIOS::ShowLeaderboardUI( const FString& LeaderboardName )
{
	extern CORE_API void IOSShowLeaderboardUI(const FString& CategoryName);
	IOSShowLeaderboardUI(LeaderboardName);
	return true;
}

bool FOnlineExternalUIIOS::ShowWebURL(const FString& Url, const FShowWebUrlParams& ShowParams, const FOnShowWebUrlClosedDelegate& Delegate)
{
	FPlatformProcess::LaunchURL(*Url, nullptr, nullptr);

	return true;
}

bool FOnlineExternalUIIOS::CloseWebURL()
{
	return false;
}

bool FOnlineExternalUIIOS::ShowProfileUI(const FUniqueNetId& Requestor, const FUniqueNetId& Requestee, const FOnProfileUIClosedDelegate& Delegate)
{
	if (Delegate.IsBound())
	{
		UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("Game Center does not support delegate for notification of profile ui closure."));
	}

	extern CORE_API bool IOSShowDashboardUI();
	return IOSShowDashboardUI();
}

bool FOnlineExternalUIIOS::ShowAccountUpgradeUI(const FUniqueNetId& UniqueId)
{
	return false;
}

bool FOnlineExternalUIIOS::ShowStoreUI(int32 LocalUserNum, const FShowStoreParams& ShowParams, const FOnShowStoreUIClosedDelegate& Delegate)
{
	return false;
}

bool FOnlineExternalUIIOS::ShowSendMessageUI(int32 LocalUserNum, const FShowSendMessageParams& ShowParams, const FOnShowSendMessageUIClosedDelegate& Delegate)
{
	return false;
}

void FOnlineExternalUIIOS::OnLoginComplete(int ControllerIndex, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& ErrorString)
{
	FOnlineError Error(bWasSuccessful);
	Error.SetFromErrorCode(ErrorString);

    FOnlineIdentityIOS* IdentityInterface = static_cast<FOnlineIdentityIOS*>(Subsystem->GetIdentityInterface().Get());
    FUniqueNetIdIOSPtr UniqueNetId;
    if (bWasSuccessful)
    {
		const FString PlayerId(FOnlineSubsystemIOS::GetPlayerId(IdentityInterface->GetLocalGameCenterUser()));
        UniqueNetId = FUniqueNetIdIOS::Create(PlayerId);
    }
    CopiedDelegate.ExecuteIfBound(UniqueNetId, ControllerIndex, Error);

	check(IdentityInterface != nullptr);
	IdentityInterface->ClearOnLoginCompleteDelegate_Handle(ControllerIndex, CompleteDelegate);
}
