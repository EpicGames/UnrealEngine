// Copyright Epic Games, Inc. All Rights Reserved.


#include "OnlineFriendsInterfaceIOS.h"
#include "OnlineSubsystemIOS.h"
#include "OnlineError.h"

// FOnlineFriendIOS

FUniqueNetIdRef FOnlineFriendIOS::GetUserId() const
{
	return UserId;
}

FString FOnlineFriendIOS::GetRealName() const
{
	FString Result;
	GetAccountData(TEXT("nickname"), Result);
	return Result;
}

FString FOnlineFriendIOS::GetDisplayName(const FString& Platform) const
{
	FString Result;
	GetAccountData(TEXT("nickname"), Result);
	return Result;
}

bool FOnlineFriendIOS::GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const
{
	return GetAccountData(AttrName, OutAttrValue);
}

EInviteStatus::Type FOnlineFriendIOS::GetInviteStatus() const
{
	return EInviteStatus::Accepted;
}

const FOnlineUserPresence& FOnlineFriendIOS::GetPresence() const
{
	return Presence;
}

// FOnlineFriendsIOS

FOnlineFriendsIOS::FOnlineFriendsIOS(FOnlineSubsystemIOS* InSubsystem):IOSSubsystem(InSubsystem)
{
	UE_LOG_ONLINE_FRIEND(Verbose, TEXT("FOnlineFriendsIOS::FOnlineFriendsIOS()"));
	check(IOSSubsystem);
	IdentityInterface = (FOnlineIdentityIOS*)InSubsystem->GetIdentityInterface().Get();
}

bool FOnlineFriendsIOS::ReadFriendsList(int32 LocalUserNum, const FString& ListName, const FOnReadFriendsListComplete& Delegate /*= FOnReadFriendsListComplete()*/)
{
	UE_LOG_ONLINE_FRIEND(Verbose, TEXT("FOnlineFriendsIOS::ReadFriendsList()"));
	Delegate.ExecuteIfBound(LocalUserNum, false, ListName, FString(TEXT("ReadFriendsList() is not supported as of UE 4.21")));
	return false;
}

bool FOnlineFriendsIOS::DeleteFriendsList(int32 LocalUserNum, const FString& ListName, const FOnDeleteFriendsListComplete& Delegate /*= FOnDeleteFriendsListComplete()*/)
{
	Delegate.ExecuteIfBound(LocalUserNum, false, ListName, FString(TEXT("DeleteFriendsList() is not supported")));
	return false;
}

bool FOnlineFriendsIOS::SendInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnSendInviteComplete& Delegate /*= FOnSendInviteComplete()*/)
{
	Delegate.ExecuteIfBound(LocalUserNum, false, FriendId, ListName, FString(TEXT("SendInvite() is not supported")));
	return false;
}

bool FOnlineFriendsIOS::AcceptInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnAcceptInviteComplete& Delegate /*= FOnAcceptInviteComplete()*/)
{
	Delegate.ExecuteIfBound(LocalUserNum, false, FriendId, ListName, FString(TEXT("AcceptInvite() is not supported")));
	return false;
}

bool FOnlineFriendsIOS::RejectInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName)
{
	TriggerOnRejectInviteCompleteDelegates(LocalUserNum, false, FriendId, ListName, FString(TEXT("RejectInvite() is not supported")));
	return false;
}

void FOnlineFriendsIOS::SetFriendAlias(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FString& Alias, const FOnSetFriendAliasComplete& Delegate /*= FOnSetFriendAliasComplete()*/)
{
	FUniqueNetIdRef FriendIdRef = FriendId.AsShared();
	IOSSubsystem->ExecuteNextTick([LocalUserNum, FriendIdRef, ListName, Delegate]()
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsIOS::SetFriendAlias is currently not supported"));
		Delegate.ExecuteIfBound(LocalUserNum, *FriendIdRef, ListName, FOnlineError(EOnlineErrorResult::NotImplemented));
	});
}

void FOnlineFriendsIOS::DeleteFriendAlias(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnDeleteFriendAliasComplete& Delegate)
{
	FUniqueNetIdRef FriendIdRef = FriendId.AsShared();
	IOSSubsystem->ExecuteNextTick([LocalUserNum, FriendIdRef, ListName, Delegate]()
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsIOS::DeleteFriendAlias is currently not supported"));
		Delegate.ExecuteIfBound(LocalUserNum, *FriendIdRef, ListName, FOnlineError(EOnlineErrorResult::NotImplemented));
	});
}

bool FOnlineFriendsIOS::DeleteFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName)
{
	TriggerOnDeleteFriendCompleteDelegates(LocalUserNum, false, FriendId, ListName, FString(TEXT("DeleteFriend() is not supported")));
	return false;
}

bool FOnlineFriendsIOS::GetFriendsList(int32 LocalUserNum, const FString& ListName, TArray< TSharedRef<FOnlineFriend> >& OutFriends)
{
	UE_LOG_ONLINE_FRIEND(Verbose, TEXT("FOnlineFriendsIOS::GetFriendsList()"));

	for (int32 Idx=0; Idx < CachedFriends.Num(); Idx++)
	{
		OutFriends.Add(CachedFriends[Idx]);
	}

	return true;
}

TSharedPtr<FOnlineFriend> FOnlineFriendsIOS::GetFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName)
{
	TSharedPtr<FOnlineFriend> Result;

	UE_LOG_ONLINE_FRIEND(Verbose, TEXT("FOnlineFriendsIOS::GetFriend()"));

	for (int32 Idx=0; Idx < CachedFriends.Num(); Idx++)
	{
		if (*(CachedFriends[Idx]->GetUserId()) == FriendId)
		{
			Result = CachedFriends[Idx];
			break;
		}
	}

	return Result;
}

bool FOnlineFriendsIOS::IsFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName)
{
	UE_LOG_ONLINE_FRIEND(Verbose, TEXT("FOnlineFriendsIOS::IsFriend()"));

	TSharedPtr<FOnlineFriend> Friend = GetFriend(LocalUserNum,FriendId,ListName);
	if (Friend.IsValid() &&
		Friend->GetInviteStatus() == EInviteStatus::Accepted)
	{
		return true;
	}
	return false;
}

bool FOnlineFriendsIOS::QueryRecentPlayers(const FUniqueNetId& UserId, const FString& Namespace)
{
	UE_LOG_ONLINE_FRIEND(Verbose, TEXT("FOnlineFriendsIOS::QueryRecentPlayers()"));

	TriggerOnQueryRecentPlayersCompleteDelegates(UserId, Namespace, false, TEXT("not implemented"));

	return false;
}

bool FOnlineFriendsIOS::GetRecentPlayers(const FUniqueNetId& UserId, const FString& Namespace, TArray< TSharedRef<FOnlineRecentPlayer> >& OutRecentPlayers)
{
	return false;
}

void FOnlineFriendsIOS::DumpRecentPlayers() const
{

}

bool FOnlineFriendsIOS::BlockPlayer(int32 LocalUserNum, const FUniqueNetId& PlayerId)
{
	return false;
}

bool FOnlineFriendsIOS::UnblockPlayer(int32 LocalUserNum, const FUniqueNetId& PlayerId)
{
	return false;
}

bool FOnlineFriendsIOS::QueryBlockedPlayers(const FUniqueNetId& UserId)
{
	return false;
}

bool FOnlineFriendsIOS::GetBlockedPlayers(const FUniqueNetId& UserId, TArray< TSharedRef<FOnlineBlockedPlayer> >& OutBlockedPlayers)
{
	return false;
}

void FOnlineFriendsIOS::DumpBlockedPlayers() const
{
}