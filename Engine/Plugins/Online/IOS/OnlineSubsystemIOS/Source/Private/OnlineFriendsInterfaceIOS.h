// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once


#include "Interfaces/OnlineFriendsInterface.h"
#include "OnlineSubsystemIOSTypes.h"
#include "Interfaces/OnlinePresenceInterface.h"

class FOnlineSubsystemIOS;

/**
 * Info associated with an online friend on the ios gamecenter service
 */
class FOnlineFriendIOS : 
	public FOnlineFriend
{
public:

	// FOnlineUser

	virtual FUniqueNetIdRef GetUserId() const override;
	virtual FString GetRealName() const override;
	virtual FString GetDisplayName(const FString& Platform = FString()) const override;
	virtual bool GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const override;

	// FOnlineFriend
	
	virtual EInviteStatus::Type GetInviteStatus() const override;
	virtual const FOnlineUserPresence& GetPresence() const override;

	// FOnlineFriendMcp

	/**
	 * Init/default constructor
	 */
	FOnlineFriendIOS(const FString& InUserId=TEXT("")) 
		: UserId(FUniqueNetIdIOS::Create(InUserId))
	{
	}

	/**
	 * Destructor
	 */
	virtual ~FOnlineFriendIOS()
	{
	}

	/**
	 * Get account data attribute
	 *
	 * @param Key account data entry key
	 * @param OutVal [out] value that was found
	 *
	 * @return true if entry was found
	 */
	inline bool GetAccountData(const FString& Key, FString& OutVal) const
	{
		const FString* FoundVal = AccountData.Find(Key);
		if (FoundVal != NULL)
		{
			OutVal = *FoundVal;
			return true;
		}
		return false;
	}

	/** User Id represented as a FUniqueNetId */
	FUniqueNetIdRef UserId;
	/** Any addition account data associated with the friend */
	TMap<FString, FString> AccountData;
	/** @temp presence info */
	FOnlineUserPresence Presence;
};

class FOnlineFriendsIOS :
	public IOnlineFriends
{
private:

	/** Reference to the main IOS identity */
	class FOnlineIdentityIOS* IdentityInterface;

	/** The collection of game center friends received through the GK callbacks in ReadFriendsList */
	TArray< TSharedRef<FOnlineFriendIOS> > CachedFriends;

	FOnlineSubsystemIOS * IOSSubsystem;

public:

	// IOnlineFriends

	virtual bool ReadFriendsList(int32 LocalUserNum, const FString& ListName, const FOnReadFriendsListComplete& Delegate = FOnReadFriendsListComplete()) override;
	virtual bool DeleteFriendsList(int32 LocalUserNum, const FString& ListName, const FOnDeleteFriendsListComplete& Delegate = FOnDeleteFriendsListComplete()) override;
	virtual bool SendInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName,  const FOnSendInviteComplete& Delegate = FOnSendInviteComplete()) override;
	virtual bool AcceptInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnAcceptInviteComplete& Delegate = FOnAcceptInviteComplete()) override;
 	virtual bool RejectInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) override;
	virtual void SetFriendAlias(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FString& Alias, const FOnSetFriendAliasComplete& Delegate = FOnSetFriendAliasComplete()) override;
	virtual void DeleteFriendAlias(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnDeleteFriendAliasComplete& Delegate = FOnDeleteFriendAliasComplete()) override; 
	virtual bool DeleteFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) override;
	virtual bool GetFriendsList(int32 LocalUserNum, const FString& ListName, TArray< TSharedRef<FOnlineFriend> >& OutFriends) override;
	virtual TSharedPtr<FOnlineFriend> GetFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) override;
	virtual bool IsFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) override;
	virtual bool QueryRecentPlayers(const FUniqueNetId& UserId, const FString& Namespace) override;
	virtual bool GetRecentPlayers(const FUniqueNetId& UserId, const FString& Namespace, TArray< TSharedRef<FOnlineRecentPlayer> >& OutRecentPlayers) override;
	virtual void DumpRecentPlayers() const override;
	virtual bool BlockPlayer(int32 LocalUserNum, const FUniqueNetId& PlayerId) override;
	virtual bool UnblockPlayer(int32 LocalUserNum, const FUniqueNetId& PlayerId) override;
	virtual bool QueryBlockedPlayers(const FUniqueNetId& UserId) override;
	virtual bool GetBlockedPlayers(const FUniqueNetId& UserId, TArray< TSharedRef<FOnlineBlockedPlayer> >& OutBlockedPlayers) override;
	virtual void DumpBlockedPlayers() const override;

	// FOnlineFriendsIOS

	/**
	 * Destructor
	 */
	virtual ~FOnlineFriendsIOS() {};

PACKAGE_SCOPE:

	/**
	 * Constructor
	 *
	 * @param InSubsystem - A pointer to the subsystem which has the features we need to gather friends. (Identity interface etc.)
	 */
	FOnlineFriendsIOS(FOnlineSubsystemIOS* InSubsystem);
};

typedef TSharedPtr<FOnlineFriendsIOS, ESPMode::ThreadSafe> FOnlineFriendsIOSPtr;
