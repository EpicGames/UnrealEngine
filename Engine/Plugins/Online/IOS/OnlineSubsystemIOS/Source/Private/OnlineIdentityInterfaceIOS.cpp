// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineIdentityInterfaceIOS.h"
#include "OnlineSubsystem.h"
#include "OnlineSubsystemIOS.h"
#include "OnlineError.h"
#include "IOS/IOSAppDelegate.h"
#include "IOS/IOSAsyncTask.h"

#import <GameKit/GameKit.h>
#import "OnlineAppStoreUtils.h"

FOnlineIdentityIOS::FOnlineIdentityIOS()
	: UniqueNetId(nullptr)
{
}

FOnlineIdentityIOS::FOnlineIdentityIOS(FOnlineSubsystemIOS* InSubsystem)
	: UniqueNetId(nullptr)
	, Subsystem(InSubsystem)
{
}

TSharedPtr<FUserOnlineAccount> FOnlineIdentityIOS::GetUserAccount(const FUniqueNetId& UserId) const
{
	// not implemented
	TSharedPtr<FUserOnlineAccount> Result;
	return Result;
}

TArray<TSharedPtr<FUserOnlineAccount> > FOnlineIdentityIOS::GetAllUserAccounts() const
{
	// not implemented
	TArray<TSharedPtr<FUserOnlineAccount> > Result;
	return Result;
}

bool FOnlineIdentityIOS::Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials)
{
	// FOnlineIdentityIOS does not support more than 1 local player
	if (LocalUserNum != 0 || bLoginInProgress)
	{
		return false;
	}

	bLoginInProgress = true;
	
	// All accesses to authenticateHandler should be done through main thread because it is non atomic
	dispatch_async(dispatch_get_main_queue(), ^
	{
		// Game Center state cannot be checked until authenticateHandler is set. Also, when setting authenticateHandler the Game Center
		// authentication process starts and a login screen may be shown to the user if the device is not logged into Game Center.
		// This is the reason it is set on the first login call instead of in an Init method or in the constructor
		// There is no way to show this login screen again from the app if the user discards it
		// If the device was logged into Game Center it will be shown a welcome message that will fade out after a few seconds
		// Once the authenticateHandler is set all Game Center status changes will be reported through it including first authentication
		// result. That is, if the device was already logged into Game Center when we set the authenticateHandler we will receive a notification
		// on it
		// The user can move to background at anytime and logout or change identity from device settings.
		// Logout events are reported as errors in the handler so will have a non nil Error
		// Login events are reported as non errors so will have a nill Error and [GKLocalPlayer localPlayer].isAuthenticated == YES
		if ([GKLocalPlayer localPlayer].authenticateHandler == nil)
		{
			[GKLocalPlayer localPlayer].authenticateHandler = (^(UIViewController* ViewController, NSError *Error)
			{
				if (ViewController == nil)
				{
					// Game Center notified a change of state. Identify what happened and handle the change from game thread
					FGamCenterEvent Event = GetCurrentGameCenterEvent(Error);
					
					[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
					{
						HandleGamCenterEvent(Event, bLoginInProgress);
						bLoginInProgress = false;
						return true;
					}];
				}
				else
				{
					// Game Center has provided a view controller for us to fulfill some needed actions, present it from main thread.
					UE_LOG_ONLINE_IDENTITY(Log, TEXT("Showing Game Center's provided UI for additional actions"));
					dispatch_async(dispatch_get_main_queue(), ^
					{
						[[IOSAppDelegate GetDelegate].IOSController presentViewController:ViewController animated:YES completion:nil];
					});
				}
			});
		}
		else
		{
			// If a login request is made after authenticateHandler is set just notify the current state
			// Game Center state updates will always be done through authenticateHandler
			[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
			{
				if (UniqueNetId.IsValid())
				{
					TriggerOnLoginCompleteDelegates(0, true, *UniqueNetId, TEXT(""));
				}
				else
				{
					TriggerOnLoginCompleteDelegates(0, false, *FUniqueNetIdIOS::EmptyId(), TEXT("Game Center authentication failed. Please, try login to Game Center from the iOS device settings"));
				}
				bLoginInProgress = false;
				return true;
			}];
		}
	});

	return true;
}

bool FOnlineIdentityIOS::Logout(int32 LocalUserNum)
{
	// There is no way to logout from Game Center from inside an app
	// Logout is only possible by closing session from Game Center's iOS device settings

	return false;
}

void FOnlineIdentityIOS::HandleGamCenterEvent(const FGamCenterEvent& Event, bool bTriggerLoginComplete)
{
	check(Event.HasValue() || Event.HasError()); // Just check nobody stole the value/error unintentionally in the future
	
	// Clear UniqueNetId but keep it locally
	FUniqueNetIdIOSPtr PreviousUniqueNetId = MoveTemp(UniqueNetId);
	bool bWasLoggedIn = PreviousUniqueNetId.IsValid();

	if (Event.HasValue())
	{
		UniqueNetId = Event.GetValue();
		
		if (bTriggerLoginComplete)
		{
			TriggerOnLoginCompleteDelegates(0, true, *UniqueNetId, TEXT(""));
		}

		if (!bWasLoggedIn || (*PreviousUniqueNetId != *UniqueNetId))
		{
			if (bWasLoggedIn)
			{
				UE_LOG_ONLINE_IDENTITY(Log, TEXT("Game Center device session closed for user %s"), *PreviousUniqueNetId->UniqueNetIdStr);
			}
			UE_LOG_ONLINE_IDENTITY(Log, TEXT("Game Center device session started for user %s"), *UniqueNetId->UniqueNetIdStr);

			TriggerOnLoginStatusChangedDelegates(0, bWasLoggedIn? ELoginStatus::LoggedIn : ELoginStatus::NotLoggedIn, ELoginStatus::LoggedIn, *UniqueNetId);
			TriggerOnLoginChangedDelegates(0);
		}
	}
	else
	{
		UE_LOG_ONLINE_IDENTITY(Log, TEXT("%s"), *Event.GetError());
		
		if (bTriggerLoginComplete)
		{
			TriggerOnLoginCompleteDelegates(0, false, *FUniqueNetIdIOS::EmptyId(), Event.GetError());
		}

		if (bWasLoggedIn)
		{
			UE_LOG_ONLINE_IDENTITY(Log, TEXT("Game Center device session closed for user %s"), *PreviousUniqueNetId->UniqueNetIdStr);
			TriggerOnLoginStatusChangedDelegates(0, ELoginStatus::LoggedIn, ELoginStatus::NotLoggedIn, *FUniqueNetIdIOS::EmptyId());
			TriggerOnLoginChangedDelegates(0);
		}
	}
}

FOnlineIdentityIOS::FGamCenterEvent FOnlineIdentityIOS::GetCurrentGameCenterEvent(NSError* Error)
{
	if (Error)
	{
		// Game Center notified an error. user is logged out from Game Center or there was an issue while authenticating
		NSString *errstr = [Error localizedDescription];
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Game Center reported an error: %s]"), *FString(errstr));
		
		return MakeError(TEXT("Game Center reported an authentication error"));
	}
	else if ([GKLocalPlayer localPlayer].isAuthenticated == YES)
	{
		// Game Center authentication succeeded. Check if we can use the identifier
		GKLocalPlayer* GKLocalUser = [GKLocalPlayer localPlayer];

		if ([GKPlayer respondsToSelector:@selector(scopedIDsArePersistent)] == YES)
		{
			// ID is not persistent across multiple game sessions, consider as not logged in
			return MakeError(TEXT("The user could not be authenticated with a persistent id by Game Center"));
		}

		NSString* GameCenterPlayerId = FOnlineSubsystemIOS::GetPlayerId(GKLocalUser);
		if ([GameCenterPlayerId isEqualToString: @"UnknownID"])
		{
			// Spuriously we may receive "UnknownID" as the id. In those cases ignore this value as a valid id and wait for the good one
			// This has been noticed when moving the app to background, logout and login from Game Center as a different account and move the app to foregroung
			// A good id will be received in the authenticateHandler on a subsequent event
			return MakeError(TEXT("UnknownID player id received. If a proper id is not received on following login attempt try logout and login from Game Center from the iOS device settings"));
		}

		FString PlayerId(GameCenterPlayerId);
		UE_LOG_ONLINE_IDENTITY(Log, TEXT("Active user for Game Center is %s"), *PlayerId);

		return MakeValue(FUniqueNetIdIOS::Create(MoveTemp(PlayerId)));
	}
	else
	{
		// According to documentation this should not happen
		return MakeError(TEXT("The user could not be authenticated by Game Center"));
	}
}

bool FOnlineIdentityIOS::AutoLogin(int32 LocalUserNum)
{
	return Login( LocalUserNum, FOnlineAccountCredentials() );
}

ELoginStatus::Type FOnlineIdentityIOS::GetLoginStatus(int32 LocalUserNum) const
{
	// FOnlineIdentityIOS does not support more than 1 local player
	if (LocalUserNum != 0)
	{
		return ELoginStatus::NotLoggedIn;
	}
	
	return UniqueNetId.IsValid()? ELoginStatus::LoggedIn : ELoginStatus::NotLoggedIn;
}

ELoginStatus::Type FOnlineIdentityIOS::GetLoginStatus(const FUniqueNetId& UserId) const 
{
	return UniqueNetId.IsValid() && UserId == *UniqueNetId? ELoginStatus::LoggedIn : ELoginStatus::NotLoggedIn;
}

FUniqueNetIdPtr FOnlineIdentityIOS::GetUniquePlayerId(int32 LocalUserNum) const
{
	// FOnlineIdentityIOS does not support more than 1 local player
	if (LocalUserNum != 0)
	{
		return nullptr;
	}
	
	return UniqueNetId;
}

FUniqueNetIdPtr FOnlineIdentityIOS::CreateUniquePlayerId(uint8* Bytes, int32 Size)
{
	if (Bytes && Size == sizeof(uint64))
	{
		int32 StrLen = FCString::Strlen((TCHAR*)Bytes);
		if (StrLen > 0)
		{
			FString StrId((TCHAR*)Bytes);
			return FUniqueNetIdIOS::Create(StrId);
		}
	}
	
	return NULL;
}

FUniqueNetIdPtr FOnlineIdentityIOS::CreateUniquePlayerId(const FString& Str)
{
	return FUniqueNetIdIOS::Create(Str);
}

FString FOnlineIdentityIOS::GetPlayerNickname(int32 LocalUserNum) const
{
	// FOnlineIdentityIOS does not support more than 1 local player
	if (LocalUserNum != 0)
	{
		return FString();
	}
	
	GKLocalPlayer* LocalPlayer = GetLocalGameCenterUser();
	
	if (LocalPlayer != nil)
	{
		NSString* PersonaName = [LocalPlayer alias];
		
		if (PersonaName != nil)
		{
			return FString(PersonaName);
		}
	}

	return FString();
}

FString FOnlineIdentityIOS::GetPlayerNickname(const FUniqueNetId& UserId) const 
{
	if (UniqueNetId.IsValid() && UserId == *UniqueNetId)
	{
		return GetPlayerNickname(0);
	}

	return FString();
}

FString FOnlineIdentityIOS::GetAuthToken(int32 LocalUserNum) const
{
	FString ResultToken;
	return ResultToken;
}

void FOnlineIdentityIOS::RevokeAuthToken(const FUniqueNetId& UserId, const FOnRevokeAuthTokenCompleteDelegate& Delegate)
{
	UE_LOG_ONLINE_IDENTITY(Display, TEXT("FOnlineIdentityIOS::RevokeAuthToken not implemented"));
	FUniqueNetIdRef UserIdRef(UserId.AsShared());
	Subsystem->ExecuteNextTick([UserIdRef, Delegate]()
	{
		Delegate.ExecuteIfBound(*UserIdRef, FOnlineError(FString(TEXT("RevokeAuthToken not implemented"))));
	});
}

void FOnlineIdentityIOS::GetUserPrivilege(const FUniqueNetId& UserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate& Delegate, EShowPrivilegeResolveUI ShowResolveUI)
{
	if (Privilege == EUserPrivileges::CanPlayOnline)
	{
		FUniqueNetIdRef SharedUserId = UserId.AsShared();
		FOnQueryAppBundleIdResponse completionDelegate = FOnQueryAppBundleIdResponse::CreateLambda([this, SharedUserId, Privilege, Delegate](NSDictionary* ResponseDict)
		{
			UE_LOG_ONLINE_IDENTITY(Log, TEXT("GetUserPrivilege Complete"));
																									   
			uint32 Result = (uint32)EPrivilegeResults::GenericFailure;
			if (ResponseDict != nil && [ResponseDict[@"resultCount"] integerValue] == 1)
			{
				// Get local bundle information
				NSDictionary* infoDictionary = [[NSBundle mainBundle] infoDictionary];
				FString localAppId = FString(infoDictionary[@"CFBundleIdentifier"]);
				FString localVersionString = FString(infoDictionary[@"CFBundleShortVersionString"]);
				UE_LOG_ONLINE_IDENTITY(Log, TEXT("Local: %s %s"), *localAppId, *localVersionString);

				// Get remote bundle information
				FString remoteAppId = FString([[[ResponseDict objectForKey:@"results"] objectAtIndex:0] objectForKey:@"bundleId"]);
				FString remoteVersionString = FString([[[ResponseDict objectForKey:@"results"] objectAtIndex:0] objectForKey:@"version"]);
				UE_LOG_ONLINE_IDENTITY(Log, TEXT("Remote: %s %s"), *remoteAppId, *remoteVersionString);

				if (localAppId == remoteAppId)
				{
					TArray<FString> LocalVersionParts;
					localVersionString.ParseIntoArray(LocalVersionParts, TEXT("."));

					TArray<FString> RemoteVersionParts;
					remoteVersionString.ParseIntoArray(RemoteVersionParts, TEXT("."));

					if (LocalVersionParts.Num() >= 2 &&
						RemoteVersionParts.Num() >= 2)
					{
						Result = (uint32)EPrivilegeResults::NoFailures;

						if (LocalVersionParts[0] != RemoteVersionParts[0] ||
							LocalVersionParts[1] != RemoteVersionParts[1])
						{
							UE_LOG_ONLINE_IDENTITY(Log, TEXT("Needs Update"));
							Result = (uint32)EPrivilegeResults::RequiredPatchAvailable;
						}
						else
						{
							const FString LocalHotfixVersion = LocalVersionParts.IsValidIndex(2) ? LocalVersionParts[2] : TEXT("0");
							const FString RemoteHotfixVersion = RemoteVersionParts.IsValidIndex(2) ? RemoteVersionParts[2] : TEXT("0");

							if (LocalHotfixVersion != RemoteHotfixVersion)
							{
								UE_LOG_ONLINE_IDENTITY(Log, TEXT("Needs Update"));
								Result = (uint32)EPrivilegeResults::RequiredPatchAvailable;
							}
							else
							{
								UE_LOG_ONLINE_IDENTITY(Log, TEXT("Does NOT Need Update"));
							}
						}
					}
				}
				else
				{
					UE_LOG_ONLINE_IDENTITY(Log, TEXT("BundleId does not match local bundleId"));
				}
			}
			else
			{
				UE_LOG_ONLINE_IDENTITY(Log, TEXT("GetUserPrivilege invalid response"));
			}

			Subsystem->ExecuteNextTick([Delegate, SharedUserId, Privilege, Result]()
			{
				Delegate.ExecuteIfBound(*SharedUserId, Privilege, Result);
			});
		});
		
		FAppStoreUtils* AppStoreUtils = Subsystem->GetAppStoreUtils();
		if (AppStoreUtils)
		{
			[AppStoreUtils queryAppBundleId: completionDelegate];
		}
	 }
	 else
	 {
		  Delegate.ExecuteIfBound(UserId, Privilege, (uint32)EPrivilegeResults::NoFailures);
	 }
}

FPlatformUserId FOnlineIdentityIOS::GetPlatformUserIdFromUniqueNetId(const FUniqueNetId& InUniqueNetId) const
{
	if (UniqueNetId.IsValid() && (*UniqueNetId == InUniqueNetId))
	{
		return GetPlatformUserIdFromLocalUserNum(0);
	}

	return PLATFORMUSERID_NONE;
}

FString FOnlineIdentityIOS::GetAuthType() const
{
	return TEXT("");
}

GKLocalPlayer* FOnlineIdentityIOS::GetLocalGameCenterUser() const
{
	return UniqueNetId.IsValid() ? [GKLocalPlayer localPlayer] : nil;
}
