// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

@class GKLocalPlayer;
@class NSError;

#include "OnlineSubsystemIOSTypes.h"
#include "Interfaces/OnlineIdentityInterface.h"
#include "Templates/ValueOrError.h"
#include "OnlineSubsystemIOSPackage.h"

class FOnlineIdentityIOS :
	public IOnlineIdentity
{
private:
	/** UID for this identity */
	FUniqueNetIdIOSPtr UniqueNetId;
	FOnlineSubsystemIOS* Subsystem;
    
    static_assert(MAX_LOCAL_PLAYERS == 1, "FOnlineIdentityIOS does not support more than 1 local player");
    bool bLoginInProgress = false;
    
	FOnlineIdentityIOS();

    using FGamCenterEvent = TValueOrError<FUniqueNetIdIOSPtr, FString>;
    
    static FGamCenterEvent GetCurrentGameCenterEvent(NSError* Error);
    void HandleGamCenterEvent(const FGamCenterEvent& LoginResult, bool bTriggerLoginComplete);
    
PACKAGE_SCOPE:

	/**
	 * Default Constructor
	 */
	FOnlineIdentityIOS(FOnlineSubsystemIOS* InSubsystem);

public:

	//~ Begin IOnlineIdentity Interface
	virtual bool Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials) override;
	virtual bool Logout(int32 LocalUserNum) override;
	virtual bool AutoLogin(int32 LocalUserNum) override;
	virtual TSharedPtr<FUserOnlineAccount> GetUserAccount(const FUniqueNetId& UserId) const override;
	virtual TArray<TSharedPtr<FUserOnlineAccount> > GetAllUserAccounts() const override;
	virtual FUniqueNetIdPtr GetUniquePlayerId(int32 LocalUserNum) const override;
	virtual FUniqueNetIdPtr CreateUniquePlayerId(uint8* Bytes, int32 Size) override;
	virtual FUniqueNetIdPtr CreateUniquePlayerId(const FString& Str) override;
	virtual ELoginStatus::Type GetLoginStatus(int32 LocalUserNum) const override;
	virtual ELoginStatus::Type GetLoginStatus(const FUniqueNetId& UserId) const override;
	virtual FString GetPlayerNickname(int32 LocalUserNum) const override;
	virtual FString GetPlayerNickname(const FUniqueNetId& UserId) const override;
	virtual FString GetAuthToken(int32 LocalUserNum) const override;
	virtual void RevokeAuthToken(const FUniqueNetId& UserId, const FOnRevokeAuthTokenCompleteDelegate& Delegate) override;
	virtual void GetUserPrivilege(const FUniqueNetId& UserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate& Delegate, EShowPrivilegeResolveUI ShowResolveUI=EShowPrivilegeResolveUI::Default) override;
	virtual FPlatformUserId GetPlatformUserIdFromUniqueNetId(const FUniqueNetId& UniqueNetId) const override;
	virtual FString GetAuthType() const override;
	//~ End IOnlineIdentity Interface

public:

	/**
	 * Destructor
	 */
	virtual ~FOnlineIdentityIOS() {};
	

	/**
	 * Get a reference to the GKLocalPlayer
	 *
	 * @return - The game center local player
	 */
    GKLocalPlayer* GetLocalGameCenterUser() const;
};


typedef TSharedPtr<FOnlineIdentityIOS, ESPMode::ThreadSafe> FOnlineIdentityIOSPtr;
