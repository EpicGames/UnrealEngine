// Copyright Epic Games, Inc. All Rights Reserved.

// Module includes
#include "OnlineLeaderboardsInterfaceIOS.h"

#include "Apple/AppleStringUtils.h"

#include "OnlineSubsystemIOS.h"

// GameCenter includes
#include <GameKit/GKLeaderboard.h>
#include <GameKit/GKScore.h>
#include <GameKit/GKLocalPlayer.h>

#include "IOS/IOSAsyncTask.h"

FOnlineLeaderboardsIOS::FOnlineLeaderboardsIOS(FOnlineSubsystemIOS* InSubsystem)
{
	UE_LOG_ONLINE_LEADERBOARD(Display, TEXT("FOnlineLeaderboardsIOS::FOnlineLeaderboardsIOS()"));

	// Cache a reference to the OSS Identity and Friends interfaces, we will need these when we are performing leaderboard actions
	IdentityInterface = (FOnlineIdentityIOS*)InSubsystem->GetIdentityInterface().Get();
	check(IdentityInterface);

	FriendsInterface = (FOnlineFriendsIOS*)InSubsystem->GetFriendsInterface().Get();
	check(FriendsInterface);

	UnreportedScores = nil;
}


FOnlineLeaderboardsIOS::~FOnlineLeaderboardsIOS()
{
	if(UnreportedScores)
	{
		[UnreportedScores release];
		UnreportedScores = nil;
	}
    
    if (CachedLeaderboard)
    {
        [CachedLeaderboard release];
        CachedLeaderboard = nil;
    }
    
    if (LeaderboardPlayer)
    {
        [LeaderboardPlayer release];
        LeaderboardPlayer = nil;
    }
    
}

bool FOnlineLeaderboardsIOS::ReadLeaderboardCompletionDelegate(NSArray* players, FOnlineLeaderboardReadRef& InReadObject)
{
    auto ReadObject = InReadObject;
	bool bTriggeredReadRequest = true;

    CachedLeaderboard = [GKLeaderboard alloc];
        [CachedLeaderboard loadEntriesForPlayers:players timeScope:GKLeaderboardTimeScopeAllTime completionHandler: ^(GKLeaderboardEntry *entries, NSArray<GKLeaderboardEntry *> *scores, NSError *Error)
         {
            bReadLeaderboardFinished = true;

            bool bWasSuccessful = (Error == nil) && [scores count] > 0;
                        
			UE_LOG_ONLINE_LEADERBOARD(Display, TEXT("FOnlineLeaderboardsIOS::loadScoresWithCompletionHandler() - %s"), (bWasSuccessful ? TEXT("Success!") : TEXT("Failed!, no scores retrieved")));
		
            if (bWasSuccessful)
            {
                bWasSuccessful = [scores count] > 0;
                for (GKLeaderboardEntry *entry in scores)
                {
                    FString PlayerIDString;
                        
                    if ([entry respondsToSelector:@selector(player)] == YES)
                    {
                        PlayerIDString = FString(FOnlineSubsystemIOS::GetPlayerId(entry.player));
                        LeaderboardPlayer = entry.player;
                    }
                        
                    UE_LOG_ONLINE_LEADERBOARD(Display, TEXT("----------------------------------------------------------------"));
                    UE_LOG_ONLINE_LEADERBOARD(Display, TEXT("PlayerId: %s"), *PlayerIDString);
                    UE_LOG_ONLINE_LEADERBOARD(Display, TEXT("Value: %lld"), entry.score);
                    UE_LOG_ONLINE_LEADERBOARD(Display, TEXT("----------------------------------------------------------------"));
                        
                    FUniqueNetIdRef UserId = FUniqueNetIdIOS::Create(PlayerIDString);
                        
                    FOnlineStatsRow* UserRow = ReadObject.Get().FindPlayerRecord(UserId.Get());
                    if (UserRow == NULL)
                    {
                        UserRow = new (ReadObject->Rows) FOnlineStatsRow(PlayerIDString, UserId);
                    }
                        
                    for (int32 StatIdx = 0; StatIdx < ReadObject->ColumnMetadata.Num(); StatIdx++)
                    {
                        const FColumnMetaData& ColumnMeta = ReadObject->ColumnMetadata[StatIdx];
                        
                        switch (ColumnMeta.DataType)
                        {
                            case EOnlineKeyValuePairDataType::Int32:
                            {
                                int32 Value = entry.score;
                                UserRow->Columns.Add(ColumnMeta.ColumnName, FVariantData(Value));
                                bWasSuccessful = true;
                                break;
                            }
                        
                            default:
                            {
                                UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Unsupported key value pair during retrieval from GameCenter %s"), *ColumnMeta.ColumnName);
                                break;
                            }
                        }
                    }
                }
            }
            else if (Error)
            {
				LeaderboardPlayer = nil;
			
                // if we have failed to read the leaderboard then report this
                NSDictionary *userInfo = [Error userInfo];
                NSString *errstr = [[userInfo objectForKey : NSUnderlyingErrorKey] localizedDescription];
                UE_LOG_ONLINE_LEADERBOARD(Display, TEXT("FOnlineLeaderboardsIOS::loadScoresWithCompletionHandler() - Failed to read leaderboard with error: [%s]"), *FString(errstr));
                UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("You should check that the leaderboard name matches that of one in ITunesConnect"));
            }
                        
            // Report back to the game thread whether this succeeded.
            [FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
             {
                ReadObject->ReadState = bWasSuccessful ? EOnlineAsyncTaskState::Done : EOnlineAsyncTaskState::Failed;
                TriggerOnLeaderboardReadCompleteDelegates(bWasSuccessful);
                return true;
             }];
        }];
    
    return bTriggeredReadRequest;
}

bool FOnlineLeaderboardsIOS::ReadLeaderboards(const TArray< FUniqueNetIdRef >& Players, FOnlineLeaderboardReadRef& InReadObject)
{
	__block FOnlineLeaderboardReadRef ReadObject = InReadObject;

	UE_LOG_ONLINE_LEADERBOARD(Display, TEXT("FOnlineLeaderboardsIOS::ReadLeaderboards()"));

    bReadLeaderboardFinished = false;
	ReadObject->ReadState = EOnlineAsyncTaskState::Failed;
	ReadObject->Rows.Empty();
	
	if ((IdentityInterface != nullptr) && (IdentityInterface->GetLocalGameCenterUser() != NULL) && IdentityInterface->GetLocalGameCenterUser().isAuthenticated)
	{
		ReadObject->ReadState = EOnlineAsyncTaskState::InProgress;

		// Populate a list of id's for our friends which we want to look up.
		NSMutableArray* FriendIds = [NSMutableArray arrayWithCapacity: (Players.Num() + 1)];
		
		// Add the local player to the list of ids to look up.
		FUniqueNetIdPtr LocalPlayerUID = IdentityInterface->GetUniquePlayerId(0);
		check(LocalPlayerUID.IsValid());

		FriendIds[0] = [NSString stringWithFString:LocalPlayerUID->ToString()];

		// Add the other requested players
		for (int32 FriendIdx = 0; FriendIdx < Players.Num(); FriendIdx++)
		{
			FriendIds[FriendIdx + 1] = [NSString stringWithFString:Players[FriendIdx]->ToString()];
		}

		// Kick off a game center read request for the list of users
                 GKLocalPlayer* GKLocalUser = [GKLocalPlayer localPlayer];
                 [GKLocalUser loadFriendsWithIdentifiers:(NSArray<NSString *> *)FriendIds completionHandler:^(NSArray<GKPlayer *> *players, NSError *Error)
		 {
			bool bWasSuccessful = (Error == nil) && [players count] > 0;
			if (bWasSuccessful)
			{
				bWasSuccessful = [players count] > 0;
			}
			// even if not successful, need to call the delegate to initialize Leaderboard and LocalPlayer
			ReadLeaderboardCompletionDelegate(players, ReadObject);
		}];
	}
	else
	{
		return false;
	}

	return true;
}


bool FOnlineLeaderboardsIOS::ReadLeaderboardsForFriends(int32 LocalUserNum, FOnlineLeaderboardReadRef& ReadObject)
{
	UE_LOG_ONLINE_LEADERBOARD(Display, TEXT("FOnlineLeaderboardsIOS::ReadLeaderboardsForFriends()"));
	if( IdentityInterface->GetLocalGameCenterUser() != NULL && IdentityInterface->GetLocalGameCenterUser().isAuthenticated )
	{
		// Gather the friends from the local players game center friends list and perform a read request for these
		TArray< TSharedRef<FOnlineFriend> > Friends;
		FriendsInterface->GetFriendsList( 0, EFriendsLists::ToString(EFriendsLists::Default), Friends );

		TArray< FUniqueNetIdRef > FriendIds;
		for( int32 Idx = 0; Idx < Friends.Num(); Idx++ )
		{
			FriendIds.Add( Friends[ Idx ]->GetUserId() );
		}
		ReadLeaderboards( FriendIds, ReadObject );
	}
	
	return true;
}

bool FOnlineLeaderboardsIOS::ReadLeaderboardsAroundRank(int32 Rank, uint32 Range, FOnlineLeaderboardReadRef& ReadObject)
{
	UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("FOnlineLeaderboardsIOS::ReadLeaderboardsAroundRank is currently not supported."));
	return false;
}
bool FOnlineLeaderboardsIOS::ReadLeaderboardsAroundUser(FUniqueNetIdRef Player, uint32 Range, FOnlineLeaderboardReadRef& ReadObject)
{
	UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("FOnlineLeaderboardsIOS::ReadLeaderboardsAroundUser is currently not supported."));
	return false;
}

void FOnlineLeaderboardsIOS::FreeStats(FOnlineLeaderboardRead& ReadObject)
{
	UE_LOG_ONLINE_LEADERBOARD(Display, TEXT("FOnlineLeaderboardsIOS::FreeStats()"));
	// not implemented for gc leaderboards
}


bool FOnlineLeaderboardsIOS::WriteLeaderboards(const FName& SessionName, const FUniqueNetId& Player, FOnlineLeaderboardWrite& WriteObject)
{
	UE_LOG_ONLINE_LEADERBOARD(Display, TEXT("FOnlineLeaderboardsIOS::WriteLeaderboards()"));
	bool bWroteAnyLeaderboard = false;

	// Make sure we have storage space for scores
	if (UnreportedScores == nil)
	{
		UnreportedScores = [[NSMutableArray alloc] initWithCapacity : WriteObject.Properties.Num()];
	}

	//@TODO: Note: The array of leaderboard names is ignored, because they offer no data.
	// Instead the stat names are used as the leaderboard names for iOS for now.  This whole API needs rework!

	// Queue up the leaderboard stat writes
	for (FStatPropertyArray::TConstIterator It(WriteObject.Properties); It; ++It)
	{
		// Access the stat and the value.
		const FVariantData& Stat = It.Value();

		FString LeaderboardName(It.Key());
		NSString* Category = [NSString stringWithFString:LeaderboardName];

		bool bIsValidScore = false;

        int32 Value;

		// Setup the score with the value we are writing from the variant type
		switch (Stat.GetType())
		{
			case EOnlineKeyValuePairDataType::Int32:
			{
				Stat.GetValue(Value);
					
				bIsValidScore = true;
				break;
			}

			default:
			{
				UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("FOnlineLeaderboardsIOS::WriteLeaderboards(Leaderboard: %s) Invalid data type (only Int32 is currently supported)"), *LeaderboardName);
				break;
			}
		}

		if (bIsValidScore)
		{
			UE_LOG_ONLINE_LEADERBOARD(Display, TEXT("FOnlineLeaderboardsIOS::WriteLeaderboards() Queued score %d on leaderboard %s"), Value, *LeaderboardName);

			[UnreportedScores addObject:[NSNumber numberWithInteger:Value]];
			bWroteAnyLeaderboard = true;
		}
	}
	
	// Return whether any stat was cached.
	return bWroteAnyLeaderboard;
}


bool FOnlineLeaderboardsIOS::FlushLeaderboards(const FName& SessionName)
{
    UE_LOG_ONLINE_LEADERBOARD(Display, TEXT("FOnlineLeaderboardsIOS::FlushLeaderboards()"));
    
    
    int SleepLimit = 30;
    while(!bReadLeaderboardFinished && SleepLimit != 0)
    {
        usleep(100000);
        --SleepLimit;
    }
    if (SleepLimit == 0)
    {
        UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Leaderboard Could not be flusshed"));
        return false;
    }
    
    bool bBeganFlushingScores = false;
    
    if ((IdentityInterface->GetLocalGameCenterUser() != NULL) && IdentityInterface->GetLocalGameCenterUser().isAuthenticated)
    {
        const int32 UnreportedScoreCount = UnreportedScores.count;
        bBeganFlushingScores = UnreportedScoreCount > 0;
        
		if (bBeganFlushingScores)
		{
			NSMutableArray *ArrayCopy = [[NSMutableArray alloc] initWithArray:UnreportedScores];
			
			[UnreportedScores release];
			UnreportedScores = nil;
			
            if (LeaderboardPlayer != nil)
            {
			dispatch_async(dispatch_get_main_queue(),
			^{
					for (NSNumber* scoreReport in ArrayCopy)
					{
						NSInteger ScoreReportInt = [scoreReport integerValue];
						
						[CachedLeaderboard submitScore:ScoreReportInt context:0 player:LeaderboardPlayer completionHandler: ^ (NSError *error)
						 {
							// Tell whoever was listening that we have written (or failed to write) to the leaderboard
							bool bSucceeded = error == NULL;
							if (bSucceeded)
							{
								UE_LOG_ONLINE_LEADERBOARD(Display, TEXT("Flushed %d scores to Game Center"), UnreportedScoreCount);
							}
							else
							{
								UE_LOG_ONLINE_LEADERBOARD(Display, TEXT("Error while flushing scores (code %lld)"), [error code]);
							}
							
							// Report back to the game thread whether this succeeded.
							[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
							 {
								TriggerOnLeaderboardFlushCompleteDelegates(SessionName, bSucceeded);
								return true;
							}];
						}];
					}
                [ArrayCopy release];
                });
				}
            else
            {
				[ArrayCopy release];
            }
		}
	
        // If we didn't begin writing to the leaderboard we should still notify whoever was listening.
        if (!bBeganFlushingScores)
        {
            TriggerOnLeaderboardFlushCompleteDelegates(SessionName, false);
            UE_LOG_ONLINE_LEADERBOARD(Display, TEXT("Failed to flush scores to leaderboard"));
        }
        
    }
    return bBeganFlushingScores;
}


bool FOnlineLeaderboardsIOS::WriteOnlinePlayerRatings(const FName& SessionName, int32 LeaderboardId, const TArray<FOnlinePlayerScore>& PlayerScores)
{
	UE_LOG_ONLINE_LEADERBOARD(Display, TEXT("FOnlineLeaderboardsIOS::WriteOnlinePlayerRatings()"));
	// not implemented for gc leaderboards
	
	return false;
}
