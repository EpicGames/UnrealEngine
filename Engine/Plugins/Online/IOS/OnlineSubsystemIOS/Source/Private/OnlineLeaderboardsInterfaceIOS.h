// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineLeaderboardInterface.h"
#include "OnlineSubsystemIOSTypes.h"

#include <GameKit/GKLeaderboard.h>
#include <GameKit/GKPlayer.h>

class FOnlineLeaderboardsIOS : public IOnlineLeaderboards
{
private:
	class FOnlineIdentityIOS* IdentityInterface;

	class FOnlineFriendsIOS* FriendsInterface;

	NSMutableArray *UnreportedScores;
    GKPlayer *LeaderboardPlayer;
    GKLeaderboard *CachedLeaderboard;
	bool bReadLeaderboardFinished;
    
    bool ReadLeaderboardCompletionDelegate(NSArray* players, FOnlineLeaderboardReadRef& ReadObject);
    
PACKAGE_SCOPE:

	FOnlineLeaderboardsIOS(FOnlineSubsystemIOS* InSubsystem);

public:

	virtual ~FOnlineLeaderboardsIOS();

	//~ Begin IOnlineLeaderboards Interface
	virtual bool ReadLeaderboards(const TArray< FUniqueNetIdRef >& Players, FOnlineLeaderboardReadRef& ReadObject) override;
	virtual bool ReadLeaderboardsForFriends(int32 LocalUserNum, FOnlineLeaderboardReadRef& ReadObject) override;
	virtual bool ReadLeaderboardsAroundRank(int32 Rank, uint32 Range, FOnlineLeaderboardReadRef& ReadObject) override;
	virtual bool ReadLeaderboardsAroundUser(FUniqueNetIdRef Player, uint32 Range, FOnlineLeaderboardReadRef& ReadObject) override;
	virtual void FreeStats(FOnlineLeaderboardRead& ReadObject) override;
	virtual bool WriteLeaderboards(const FName& SessionName, const FUniqueNetId& Player, FOnlineLeaderboardWrite& WriteObject) override;
	virtual bool FlushLeaderboards(const FName& SessionName) override;
	virtual bool WriteOnlinePlayerRatings(const FName& SessionName, int32 LeaderboardId, const TArray<FOnlinePlayerScore>& PlayerScores) override;
	//~ End IOnlineLeaderboards Interface

};

typedef TSharedPtr<FOnlineLeaderboardsIOS, ESPMode::ThreadSafe> FOnlineLeaderboardsIOSPtr;
