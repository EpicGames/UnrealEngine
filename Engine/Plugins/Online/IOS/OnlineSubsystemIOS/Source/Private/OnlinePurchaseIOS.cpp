// Copyright Epic Games, Inc. All Rights Reserved.

#if UE_WITH_STORE_KIT

#include "OnlinePurchaseIOS.h"
#include "OnlineError.h"
#include "OnlineSubsystemIOS.h"
#include "Misc/CommandLine.h"
#include "Stats/Stats.h"
#include "Async/Async.h"

#import <StoreKit/SKError.h>
#import <StoreKit/SKReceiptRefreshRequest.h>
#import <StoreKit/SKPaymentTransaction.h>
#import <StoreKit/SKPayment.h>
#import <StoreKit/SKPaymentQueue.h>

#include "IOS/IOSAsyncTask.h"

#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
#include "MarketplaceKit.h"
#endif // !PLATFORM_TVOS && !PLATFORM_VISIONOS

#define LOCTEXT_NAMESPACE "OnlineSubsystemIOS"

namespace OnlinePurchaseIOSPrivate
{
    TAutoConsoleVariable<int32> CVarSimluateAskToBuy(TEXT("OSS.AskToBuy"),
                                                     0,
                                                     TEXT("Simulate ask to buy in iOS\n")
                                                     TEXT("1 enable, 0 disable"),
                                                     ECVF_Default);

    /**
     * Convert an Apple SKPaymentTransaction receipt into a string
     *
     * @return hex encoded string with opaque data representing a completed transaction
     */

	FString ConvertReceiptToString()
	{
		FString ReceiptData;
		
		NSURL* nsReceiptUrl = [[NSBundle mainBundle] appStoreReceiptURL];
		NSData* nsReceiptData = [NSData dataWithContentsOfURL : nsReceiptUrl];
		if (nsReceiptData)
		{
			NSString* nsEncodedReceiptData = [nsReceiptData base64EncodedStringWithOptions: 0];
			ReceiptData = nsEncodedReceiptData;
			UE_LOG_ONLINE_PURCHASE(VeryVerbose, TEXT("ConvertReceiptToString %s"), *ReceiptData);
		}
		else
		{
			UE_LOG_ONLINE_PURCHASE(Log, TEXT("ConvertReceiptToString: No receipt data found for transaction"));
		}
		return ReceiptData;
	}

	TSharedPtr<FString> ConvertReceiptToSharedString()
    {
		TSharedPtr<FString> ReceiptSharedData;
		FString ReceiptData = ConvertReceiptToString();
		if (!ReceiptData.IsEmpty())
		{
			ReceiptSharedData = MakeShared<FString>(MoveTemp(ReceiptData));
		}
        return ReceiptSharedData;
    }
}

/**
 * Intermediate type to communicate SKPaymentTransaction information from Objective-C to
 * to C++ implementation
 */
struct FOnlinePurchaseTransactionIOS
{
    explicit FOnlinePurchaseTransactionIOS(EPurchaseTransactionState State, SKPaymentTransaction* Transaction, TSharedPtr<FString> NativeReceipt);
    
    /** @return a string that prints useful debug information about this transaction */
    FString ToDebugString(bool VeryVerbose) const
    {
		if(VeryVerbose)
		{
			return FString::Printf(TEXT("OfferId: %s TransactionId: %s Date: %s ReceiptData: %s Error:%s"),
								   *OfferId,
								   *TransactionIdentifier,
								   *FString([NSDateFormatter localizedStringFromDate: GetPaymentTransaction().transactionDate dateStyle:NSDateFormatterShortStyle timeStyle:NSDateFormatterShortStyle]),
								   *GetReceiptData(),
								   *ErrorStr);
		}
		else
		{
			return FString::Printf(TEXT("OfferId: %s TransactionId: %s Date: %s Error:%s"),
								   *OfferId,
								   *TransactionIdentifier,
								   *FString([NSDateFormatter localizedStringFromDate: GetPaymentTransaction().transactionDate dateStyle:NSDateFormatterShortStyle timeStyle:NSDateFormatterShortStyle]),
								   *ErrorStr);
		}
    }
    /** @return transaction state */
	EPurchaseTransactionState GetTransactionState() const { return TransactionState; }
    /** @return offer id for this transaction */
    const FString& GetOfferId() const { return OfferId; }
    /** @return receipt data for this transaction */
    FString GetReceiptData() const { return ReceiptData ? *ReceiptData : FString(); }
    /** @return error string for this transaction, if applicable */
    const FString& GetErrorStr() const { return ErrorStr; }
    /** @return quantity of items requested of the same type */
    int GetQuantity() const { return Quantity; }
    /** @return transaction identifier for this purchase */
    const FString& GetTransactionIdentifier() const { return TransactionIdentifier; }
    /** @return the SKPaymentTransaction object so we can finish the transaction */
    SKPaymentTransaction* GetPaymentTransaction() const { return static_cast<SKPaymentTransaction*>(PaymentTransaction); }
private:
    
    /** iTunesConnect offer id */
    FString OfferId;
    /** Opaque store receipt data */
    TSharedPtr<FString> ReceiptData;
    /** Error on the transaction, if applicable */
    FString ErrorStr;
    /** Unique transaction identifier */
    FString TransactionIdentifier;
    /** Quantity of items requested*/
    int Quantity;
	/** The transalted transaction state */
	EPurchaseTransactionState TransactionState;
    /** Platform transaction object*/
    TRetainedObjCInstance<SKPaymentTransaction*> PaymentTransaction;
};

FOnlinePurchaseTransactionIOS::FOnlinePurchaseTransactionIOS(EPurchaseTransactionState State, SKPaymentTransaction* Transaction, TSharedPtr<FString> NativeReceipt)
    : ReceiptData(NativeReceipt)
    , TransactionIdentifier(Transaction.transactionIdentifier)
    , Quantity(0)
	, TransactionState(State)
    , PaymentTransaction(Transaction)
{
    if(Transaction.error)
    {
        FString ErrorRaw([Transaction.error localizedDescription]);
        if (!ErrorRaw.IsEmpty())
        {
            int32 ErrorCode([Transaction.error code]);
            FString ErrorDomain([Transaction.error domain]);
            ErrorStr = FString::Printf(TEXT("%s [%s:%d]"), *ErrorRaw, *ErrorDomain, ErrorCode);
        }
    }
    
    OfferId = Transaction.payment.productIdentifier;
    Quantity = Transaction.payment.quantity;
}

/**
 * Info used to cache and track order in progress.
 */
struct FOnlinePurchaseInProgressTransactionIOS
{
    FOnlinePurchaseInProgressTransactionIOS(const FPurchaseCheckoutRequest& InCheckoutRequest, const FOnPurchaseCheckoutComplete InCheckoutCompleteDelegate)
        : CheckoutRequest(InCheckoutRequest)
        , CheckoutCompleteDelegate(InCheckoutCompleteDelegate)
    {
    }
    
    bool HasRequestForProduct(const FString& ProductId) const
    {
        const TArray<FPurchaseCheckoutRequest::FPurchaseOfferEntry>& Offers = CheckoutRequest.PurchaseOffers;
        return (Offers.Num() > 0 ? Offers[0].OfferId == ProductId : false);
    }
    
    /** Checkout info for the order */
    FPurchaseCheckoutRequest CheckoutRequest;
    /** Delegate to call on completion */
    FOnPurchaseCheckoutComplete CheckoutCompleteDelegate;
};

/**
 * Proxy class that notifies updates from SKPaymentQueue to FOnlinePurchaseIOS on the game 
 * thread and invokes SKPaymentQueue methods on main thread
 */
@interface FStoreKitPurchaseProxy : NSObject<SKPaymentTransactionObserver>
{
    FOnlinePurchaseIOS* _PurchaseReceiver;
};
@end

@implementation FStoreKitPurchaseProxy

////////////////////////////////////////////////////////////////////
/// SKPaymentTransactionObserver implementation
-(void)paymentQueue: (SKPaymentQueue*)Queue updatedTransactions: (NSArray*)Transactions
{
    UE_LOG_ONLINE_PURCHASE(Log, TEXT("FStoreKitPurchaseProxy::updatedTransactions"));

	TSharedPtr<TArray<FOnlinePurchaseTransactionIOS>> IosTransactions = [self TranslateTransactions: Transactions];

	[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
	{
		if(_PurchaseReceiver)
		{
			_PurchaseReceiver->OnUpdatedTransactions(IosTransactions ? *IosTransactions : TArray<FOnlinePurchaseTransactionIOS>());
		}
		return true;
	}];
}

-(void)paymentQueue: (SKPaymentQueue*)Queue removedTransactions: (NSArray*)Transactions
{
    UE_LOG_ONLINE_PURCHASE(Log, TEXT("FStoreKitPurchaseProxy::removedTransaction"));
	
	TSharedPtr<TArray<FOnlinePurchaseTransactionIOS>> IosTransactions = [self TranslateTransactions: Transactions];
	
	[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
	{
		if(_PurchaseReceiver)
		{
			_PurchaseReceiver->OnRemovedTransactions(IosTransactions ? *IosTransactions : TArray<FOnlinePurchaseTransactionIOS>());
		}
		return true;
	}];
}

-(void)paymentQueue: (SKPaymentQueue*)Queue restoreCompletedTransactionsFailedWithError: (NSError*)Error
{
	EPurchaseTransactionState CompletionState = [FStoreKitPurchaseProxy TranslateError:Error];
    UE_LOG_ONLINE_PURCHASE(Log, TEXT("FStoreKitPurchaseProxy::restoreCompletedTransactionsFailedWithError %d"), (int32)CompletionState);

    // Notifications from SKPaymentQueue are received on main thread. Use a task to notify from game thread
    [FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
    {
        if(_PurchaseReceiver)
        {
            _PurchaseReceiver->OnQueryReceiptsComplete(false, nullptr);
        }
        return true;
    }];
}

-(void)paymentQueueRestoreCompletedTransactionsFinished: (SKPaymentQueue*)Queue
{
    UE_LOG_ONLINE_PURCHASE(Log, TEXT("FStoreKitPurchaseProxy::paymentQueueRestoreCompletedTransactionsFinished"));

	[self NotifySuccessfulQueryReceiptsComplete];
}

-(void)NotifySuccessfulQueryReceiptsComplete
{
	NSArray<SKPaymentTransaction*>* KnownTransactions = [[SKPaymentQueue defaultQueue] transactions];
	TSharedPtr<TArray<FOnlinePurchaseTransactionIOS>> IosTransactions = [self TranslateTransactions: KnownTransactions];

	// Notifications from SKPaymentQueue are received on main thread. Use a task to notify from game thread
	[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
	{
		if(_PurchaseReceiver)
		{
			_PurchaseReceiver->OnQueryReceiptsComplete(true, IosTransactions);
		}
		return true;
	}];
}

////////////////////////////////////////////////////////////////////
/// FStoreKitPurchaseProxy methods

- (id)initWithReceiver: (FOnlinePurchaseIOS*)PurchaseReceiver
{
    _PurchaseReceiver = PurchaseReceiver;

    // Notifications from SKPaymentQueue are received on main thread. Use dispatch_async on main queue to serialize operations
    dispatch_async(dispatch_get_main_queue(), ^
    {
		[[SKPaymentQueue defaultQueue] addTransactionObserver: self];
    });
    return self;
}

-(void)Shutdown
{
	[[SKPaymentQueue defaultQueue] removeTransactionObserver: self];
    _PurchaseReceiver = nullptr;
}

- (bool)CanMakePayments
{
    return [SKPaymentQueue canMakePayments] != FALSE;
}

- (void)QueryKnownReceipts: (bool)completeQuery
{
	// Notifications from SKPaymentQueue are received on main thread. Use dispatch_async on main queue to serialize operations
	dispatch_async(dispatch_get_main_queue(), ^
	{
		[self NotifySuccessfulQueryReceiptsComplete];
	});
}

+(EPurchaseTransactionState)TranslateError: (NSError*)Error
{
    EPurchaseTransactionState TranslatedError = EPurchaseTransactionState::Failed;
    switch (Error.code)
    {
        case SKErrorPaymentCancelled:
            TranslatedError = EPurchaseTransactionState::Canceled;
            break;
        case SKErrorClientInvalid:
        case SKErrorStoreProductNotAvailable:
        case SKErrorPaymentInvalid:
            TranslatedError = EPurchaseTransactionState::Invalid;
            break;
        case SKErrorPaymentNotAllowed:
            TranslatedError = EPurchaseTransactionState::NotAllowed;
            break;
    }
    return TranslatedError;
}

// Returns a TSharedPtr to avoid copying the contained TArray when captured in Obj-C blocks
-(TSharedPtr<TArray<FOnlinePurchaseTransactionIOS>>)TranslateTransactions: (NSArray<SKPaymentTransaction*>*) Transactions
{
	TSharedPtr<TArray<FOnlinePurchaseTransactionIOS>> IosTransactions;
	const int32 TransactionCount = [Transactions count];
	if (TransactionCount > 0)
	{
		IosTransactions = MakeShared<TArray<FOnlinePurchaseTransactionIOS>>();
		IosTransactions->Reserve(TransactionCount);
		TSharedPtr<FString> ReceiptSnapshot;
		
		for (SKPaymentTransaction* Transaction in Transactions)
		{
			SKPaymentTransactionState State = [Transaction transactionState];
			// The string receipt data may be big. Avoid creating it if won't be needed
			if (ReceiptSnapshot == nullptr && (State == SKPaymentTransactionStatePurchased || State == SKPaymentTransactionStateRestored))
			{
				ReceiptSnapshot = OnlinePurchaseIOSPrivate::ConvertReceiptToSharedString();
			}

			if (TOptional<FOnlinePurchaseTransactionIOS> TransactionToAdd = [self TryTranslateTransaction: Transaction withReceiptData: ReceiptSnapshot])
			{
				IosTransactions->Emplace(MoveTemp(*TransactionToAdd));
			}
		}
	}
	return IosTransactions;
}

-(TOptional<FOnlinePurchaseTransactionIOS>)TryTranslateTransaction: (SKPaymentTransaction*)Transaction withReceiptData: (TSharedPtr<FString>)ReceiptData
{
    TOptional<EPurchaseTransactionState> TranslatedState;
    SKPaymentTransaction* ActualTransaction = Transaction;
    SKPaymentTransaction* FinishTransaction = nil;
    
    switch ([Transaction transactionState])
    {
    case SKPaymentTransactionStatePurchased:
        UE_LOG_ONLINE_PURCHASE(Log, TEXT("FStoreKitPurchaseProxy::TryTranslateTransaction SKPaymentTransactionStatePurchased"));

		if (FParse::Param(FCommandLine::Get(), TEXT("disableiosredeem")))
		{
			UE_LOG_ONLINE_PURCHASE(Log, TEXT("FStoreKitPurchaseProxy::completedTransaction (disabled)"));
		}
		else
		{
			TranslatedState = EPurchaseTransactionState::Purchased;
		}
        break;
    case SKPaymentTransactionStateFailed:
        UE_LOG_ONLINE_PURCHASE(Log, TEXT("FStoreKitPurchaseProxy::TryTranslateTransaction SKPaymentTransactionStateFailed"));
        TranslatedState = [FStoreKitPurchaseProxy TranslateError:ActualTransaction.error];
        FinishTransaction = ActualTransaction;
        break;
    case SKPaymentTransactionStateRestored:
        UE_LOG_ONLINE_PURCHASE(Log, TEXT("FStoreKitPurchaseProxy::TryTranslateTransaction SKPaymentTransactionStateRestored"));
        TranslatedState = EPurchaseTransactionState::Restored;
        FinishTransaction = ActualTransaction;
        ActualTransaction = ActualTransaction.originalTransaction;
        break;
    case SKPaymentTransactionStatePurchasing:
        UE_LOG_ONLINE_PURCHASE(Log, TEXT("FStoreKitPurchaseProxy::TryTranslateTransaction SKPaymentTransactionStatePurchasing"));
        break;
    case SKPaymentTransactionStateDeferred:
        UE_LOG_ONLINE_PURCHASE(Log, TEXT("FStoreKitPurchaseProxy::TryTranslateTransaction SKPaymentTransactionStateDeferred"));
        TranslatedState = EPurchaseTransactionState::Deferred;
        break;
    default:
        UE_LOG_ONLINE_PURCHASE(Warning, TEXT("FStoreKitPurchaseProxy::TryTranslateTransaction unknown transaction state: %lld"), [Transaction transactionState]);
        break;
    }

	TOptional<FOnlinePurchaseTransactionIOS> TranslatedTransaction;
	if (TranslatedState)
	{
		TranslatedTransaction.Emplace(*TranslatedState, ActualTransaction, MoveTemp(ReceiptData));
	}
	
	// Finish transactions that should be unconditionally finish. Only leave unfinished those related to an in progress purchase so
	// licensees can finish after validation
    if(FinishTransaction)
    {
        UE_LOG_ONLINE_PURCHASE(Log, TEXT("FStoreKitPurchaseProxy::TryTranslateTransaction transaction finished"));
        [self FinalizeTransaction:FinishTransaction];
    }
	return TranslatedTransaction;
}

-(void)RestorePurchases
{
    // Notifications from SKPaymentQueue are received on main thread. Use dispatch_async on main queue to serialize operations
    dispatch_async(dispatch_get_main_queue(), ^
    {
        [[SKPaymentQueue defaultQueue] restoreCompletedTransactions];
    });
}

-(void)MakePurchase: (SKProduct*)Product withQuantity: (int)Quantity simulateAskToBuy: (bool) bAskToBuy
{
    UE_LOG_ONLINE_PURCHASE(Log, TEXT("FStoreKitPurchaseProxy::makePurchase by SKProduct"));

    // Notifications from SKPaymentQueue are received on main thread. Use dispatch_async on main queue to serialize operations
    dispatch_async(dispatch_get_main_queue(), ^
    {
        SKMutablePayment* Payment = [SKMutablePayment paymentWithProduct:Product];
        Payment.quantity = Quantity;
        // simulatesAskToBuyInSandbox is only effective while Sandbox testing
        Payment.simulatesAskToBuyInSandbox = bAskToBuy;

        [[SKPaymentQueue defaultQueue] addPayment:Payment];
    });
}

-(void)FinalizeTransaction: (SKPaymentTransaction*)Transaction
{
    UE_LOG_ONLINE_PURCHASE(Log, TEXT("FStoreKitPurchaseProxy::finalizeTransaction"));

    // Notifications from SKPaymentQueue are received on main thread. Use dispatch_async on main queue to serialize operations
    dispatch_async(dispatch_get_main_queue(), ^
    {
        [[SKPaymentQueue defaultQueue] finishTransaction:Transaction];
    });
}

@end

FOnlinePurchaseIOS::FOnlinePurchaseIOS(FOnlineSubsystemIOS* InSubsystem)
	: bQueryingReceipts(false)
	, Subsystem(InSubsystem)
{
	UE_LOG_ONLINE_PURCHASE(Log, TEXT( "FOnlinePurchaseIOS::FOnlinePurchaseIOS" ));
    StoreKitProxy = [[[FStoreKitPurchaseProxy alloc] initWithReceiver:this] autorelease];
}

////////////////////////////////////////////////////////////////////
/// FOnlinePurchaseIOS methods

FOnlinePurchaseIOS::~FOnlinePurchaseIOS()
{
    [StoreKitProxy Shutdown];
}

void FOnlinePurchaseIOS::DumpAppReceipt()
{
    FString receiptData = OnlinePurchaseIOSPrivate::ConvertReceiptToString();
    UE_LOG_ONLINE_PURCHASE(Verbose, TEXT("FOnlinePurchaseIOS::DumpAppReceipt"));
	UE_LOG_ONLINE_PURCHASE(Verbose, TEXT("%s"), *receiptData);
}

void FOnlinePurchaseIOS::GetAppReceipt(TFunction<void(const FString&)> OnComplete)
{
	AsyncTask(ENamedThreads::AnyBackgroundThreadNormalTask, [OnComplete]()
	{
		FString ReceiptData = OnlinePurchaseIOSPrivate::ConvertReceiptToString();

		AsyncTask(ENamedThreads::GameThread, [OnComplete, ReceiptData]()
		{
			OnComplete(ReceiptData);
		});
	});
}

bool FOnlinePurchaseIOS::IsAllowedToPurchase(const FUniqueNetId& /*UserId*/)
{
	bool bCanMakePurchases = [StoreKitProxy CanMakePayments];
	UE_LOG_ONLINE_PURCHASE(Verbose, TEXT("FOnlinePurchaseIOS::IsAllowedToPurchase %s"), *LexToString(bCanMakePurchases));
	return bCanMakePurchases;
}

bool FOnlinePurchaseIOS::IsAllowedToPurchase(const FUniqueNetId& UserId, FOnlineError& Error)
{
	bool bResult = IsAllowedToPurchase(UserId);
	Error = FOnlineError(bResult ? EOnlineErrorResult::Success : EOnlineErrorResult::Unknown);
	return bResult;
}

void FOnlinePurchaseIOS::Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseCheckoutComplete& Delegate)
{
	bool bStarted = false;
	FText ErrorMessage;

    if (IsAllowedToPurchase(UserId))
	{
		if (InProgressTransaction == nullptr)
		{
			FOnlineStoreIOSPtr StoreInterface = StaticCastSharedPtr<FOnlineStoreIOS>(Subsystem->GetStoreV2Interface());
			if (StoreInterface.IsValid())
			{
                int32 NumOffers = CheckoutRequest.PurchaseOffers.Num();
                
                SKProduct* Product = nil;
                if(NumOffers > 0)
                {
                    const FPurchaseCheckoutRequest::FPurchaseOfferEntry& Offer = CheckoutRequest.PurchaseOffers[0];
                    if (NumOffers > 1)
                    {
                        UE_LOG_ONLINE_PURCHASE(Warning, TEXT("StoreKit does not support multiple different products in one transaction. Only %s will be requested"), *Offer.OfferId);
                    }
                    Product = StoreInterface->GetSKProductByOfferId(Offer.OfferId);
                
                    if (Product)
                    {
                        bool bAskToBuy = false;
#if !UE_BUILD_SHIPPING
                        bAskToBuy = OnlinePurchaseIOSPrivate::CVarSimluateAskToBuy.GetValueOnGameThread() == 1;
#endif
						InProgressTransaction = MakeShared<const FOnlinePurchaseInProgressTransactionIOS>(CheckoutRequest, Delegate);
							
						const bool bOfferFinalizing = FinalizingTransactions.ContainsByPredicate(
							[OfferId = Offer.OfferId](const FKnownTransaction& Transaction)
							{
								return OfferId == FString(static_cast<SKPaymentTransaction*>(Transaction.PaymentTransaction).payment.productIdentifier);
							});
						
						if (bOfferFinalizing)
						{
							UE_LOG_ONLINE_PURCHASE(Log, TEXT("Deferring purchase of offer %s until finalization of previous purchase completes."), *Offer.OfferId);
							OnTransactionFinalized.Add(FOnTransactionFinalized::FDelegate::CreateSPLambda(InProgressTransaction.ToSharedRef(),
								[this, ProductToPurchase = TRetainedObjCInstance<SKProduct*>(Product), OfferId = Offer.OfferId, Quantity = Offer.Quantity, bAskToBuy](const FOnlinePurchaseTransactionIOS& Transaction)
								{
									if (Transaction.GetOfferId() == OfferId)
									{
										[StoreKitProxy MakePurchase:ProductToPurchase withQuantity:Quantity simulateAskToBuy:bAskToBuy];
										OnTransactionFinalized.RemoveAll(InProgressTransaction.Get());
									}
								}));
						}
						else
						{
							[StoreKitProxy MakePurchase:Product withQuantity:Offer.Quantity simulateAskToBuy:bAskToBuy];
						}
						bStarted = true;
                    }
                }
                if(!bStarted)
                {
                    ErrorMessage = NSLOCTEXT("IOSPurchase", "ErrorNoOffersSpecified", "Failed to checkout, no valid offers given.");
                }
			}
            else
            {
                ErrorMessage = NSLOCTEXT("IOSPurchase", "ErrorPurchaseInvalidInstance", "Failed to checkout, invalid FOnlineStoreIOS instance.");
            }		
        }
		else
		{
			ErrorMessage = NSLOCTEXT("IOSPurchase", "ErrorTransactionInProgress", "Failed to checkout, user has in progress transaction or is retoring transactions.");
		}
	}
	else
	{
		ErrorMessage = NSLOCTEXT("IOSPurchase", "ErrorPurchaseNotAllowed", "Failed to checkout, user not allowed to purchase.");
	}

	if (!bStarted)
	{
		TSharedRef<FPurchaseReceipt> FailReceipt = GenerateFailReceipt(CheckoutRequest);
		
        // Notify failure on next tick
		Subsystem->ExecuteNextTick([ErrorMessage, FailReceipt = FailReceipt, Delegate]()
		{
			FOnlineError Error(ErrorMessage);
			Delegate.ExecuteIfBound(Error, FailReceipt);
		});
	}
}

void FOnlinePurchaseIOS::Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseReceiptlessCheckoutComplete& Delegate)
{
	UE_LOG_ONLINE(Error, TEXT("FOnlinePurchaseIOS::Checkout (receiptless) is not currently supported"));
	Delegate.ExecuteIfBound(FOnlineError(EOnlineErrorResult::NotImplemented));
}

void FOnlinePurchaseIOS::FinalizePurchase(const FUniqueNetId& UserId, const FString& ReceiptId)
{
	UE_LOG_ONLINE_PURCHASE(Log, TEXT("FOnlinePurchaseIOS::FinalizePurchase %s %s"), *UserId.ToString(), *ReceiptId);

	const int32 TransactionToFinalizeIndex = CachedReceipts.IndexOfByPredicate([&ReceiptId](const FKnownTransaction& Entry) { return Entry.Receipt->TransactionId == ReceiptId; });
	if (TransactionToFinalizeIndex != INDEX_NONE)
	{
		// Restored transactions are already marked finalized when received but kept as known transactions so user code can process them
		if (CachedReceipts[TransactionToFinalizeIndex].Receipt->TransactionState != EPurchaseTransactionState::Restored)
		{
			[StoreKitProxy FinalizeTransaction: CachedReceipts[TransactionToFinalizeIndex].PaymentTransaction];

			// Keep track of transactions being finalized. This must complete before attempting to purchase the same offer again
			FinalizingTransactions.Add(MoveTemp(CachedReceipts[TransactionToFinalizeIndex]));
		}
		
		// Once finalized, remove from the cached receipts as a finalized transaction will not appear in the result of a query
		CachedReceipts.RemoveAt(TransactionToFinalizeIndex);
	}
	else
	{
		UE_LOG_ONLINE_PURCHASE(Warning, TEXT("FOnlinePurchaseIOS::FinalizePurchase transaction not found in CachedReceipts"));
	}
}

void FOnlinePurchaseIOS::RedeemCode(const FUniqueNetId& UserId, const FRedeemCodeRequest& RedeemCodeRequest, const FOnPurchaseRedeemCodeComplete& Delegate)
{
	FOnlineError Result;
	Delegate.ExecuteIfBound(Result, MakeShared<FPurchaseReceipt>());
}

void FOnlinePurchaseIOS::QueryReceipts(const FUniqueNetId& UserId, bool bRestoreReceipts, const FOnQueryReceiptsComplete& Delegate)
{
	bool bStarted = true;

	if (!bQueryingReceipts)
	{
		QueryReceiptsComplete = Delegate;
		bQueryingReceipts = true;
		if (bRestoreReceipts)
		{
            [StoreKitProxy RestorePurchases];
		}
		else
		{
			[StoreKitProxy QueryKnownReceipts:true];
		}
	}
	else
	{
		UE_LOG_ONLINE_PURCHASE(Log, TEXT("FOnlinePurchaseIOS::QueryReceipts already restoring transactions or checkout in progress"));
		bStarted = false;
	}

	if (!bStarted)
	{
		// Query receipts comes dynamically from the StoreKit observer
		Subsystem->ExecuteNextTick([Delegate]() {
			FOnlineError Result(false);
			Delegate.ExecuteIfBound(Result);
		});
	}
}

void FOnlinePurchaseIOS::GetReceipts(const FUniqueNetId& UserId, TArray<FPurchaseReceipt>& OutReceipts) const
{
	OutReceipts.Empty();

	for (const FKnownTransaction& Transaction: CachedReceipts)
	{
		OutReceipts.Add(*Transaction.Receipt);
	}
}

void FOnlinePurchaseIOS::OnUpdatedTransactions(const TArray<FOnlinePurchaseTransactionIOS>& Transactions)
{
	UE_LOG_ONLINE_PURCHASE(Log, TEXT("FOnlinePurchaseIOS::OnUpdatedTransactions HasInprogressTransaction: %d IsQueryingReceipts: %d"), InProgressTransaction != nullptr, bQueryingReceipts);

	bool bExpectingTransactions = InProgressTransaction != nullptr || bQueryingReceipts; 
	if (InProgressTransaction != nullptr)
	{
		struct FCheckoutResult
		{
			FOnlineError Result;
			TSharedPtr<FPurchaseReceipt> Receipt;
		};
		TOptional<FCheckoutResult> CheckoutResult;
		
		for (const FOnlinePurchaseTransactionIOS& Transaction : Transactions)
		{
			const EPurchaseTransactionState TransactionState = Transaction.GetTransactionState();
			
			if (InProgressTransaction != nullptr &&
				TransactionState != EPurchaseTransactionState::Restored &&
				InProgressTransaction->HasRequestForProduct(Transaction.GetOfferId()))
			{
				FOnlineError FinalResult;
				const FString& ErrorStr = Transaction.GetErrorStr();
				
				switch (TransactionState)
				{
					case EPurchaseTransactionState::Failed:
						FinalResult.SetFromErrorCode(TEXT("com.epicgames.purchase.failure"));
						FinalResult.ErrorMessage = !ErrorStr.IsEmpty() ? FText::FromString(ErrorStr) : LOCTEXT("IOSTransactionFailed", "TransactionFailed");
						break;
					case EPurchaseTransactionState::Canceled:
						FinalResult.SetFromErrorCode(TEXT("com.epicgames.catalog_helper.user_cancelled"));
						FinalResult.ErrorMessage = !ErrorStr.IsEmpty() ? FText::FromString(ErrorStr) : LOCTEXT("IOSTransactionCancel", "TransactionCanceled");
						break;
					case EPurchaseTransactionState::Purchased:
						FinalResult.bSucceeded = true;
						break;
					case EPurchaseTransactionState::Deferred:
						FinalResult.SetFromErrorCode(TEXT("com.epicgames.purchase.deferred"));
						FinalResult.ErrorMessage = !ErrorStr.IsEmpty() ? FText::FromString(ErrorStr) : LOCTEXT("IOSTransactionDeferred", "Transaction awaiting approval.");
						break;
					default:
						UE_LOG_ONLINE_PURCHASE(Warning, TEXT("Unexpected state after purchase %d"), (int)TransactionState);
						FinalResult.SetFromErrorCode(TEXT("com.epicgames.purchase.unexpected_state"));
						FinalResult.ErrorMessage = !ErrorStr.IsEmpty() ? FText::FromString(ErrorStr) : LOCTEXT("UnexpectedState", "Unexpected purchase result");
						break;
				}
				
				TSharedRef<FPurchaseReceipt> Receipt = GenerateReceipt(InProgressTransaction->CheckoutRequest, Transaction);
				
				if(FinalResult.bSucceeded)
				{
					AddReceiptToCache(CachedReceipts, Receipt, Transaction);
				}
				CheckoutResult = FCheckoutResult{MoveTemp(FinalResult), MoveTemp(Receipt)};
				break;
			}
		}

		if (CheckoutResult)
		{
			InProgressTransaction->CheckoutCompleteDelegate.ExecuteIfBound(CheckoutResult->Result, CheckoutResult->Receipt.ToSharedRef());
			InProgressTransaction.Reset();
		}
		else
		{
			bExpectingTransactions = bQueryingReceipts;
		}
	}
	
	if (!bExpectingTransactions)
	{
		for (const FOnlinePurchaseTransactionIOS& Transaction : Transactions)
		{
			if (Transaction.GetTransactionState() == EPurchaseTransactionState::Purchased)
			{
				UE_LOG_ONLINE_PURCHASE(Log, TEXT("Deferred transaction finished or subscription renewal transaction was received"));
				TWeakPtr<FOnlinePurchaseIOS> WeakThis = AsShared();
				Subsystem->ExecuteNextTick([WeakThis]() {
					FOnlinePurchaseIOSPtr StrongThis = WeakThis.Pin();
					if (StrongThis.IsValid())
					{
						// No user id for app store
						StrongThis->TriggerOnUnexpectedPurchaseReceiptDelegates(*FUniqueNetIdIOS::EmptyId());
					}
				});
				break;
			}
		}
	}
}

void FOnlinePurchaseIOS::OnRemovedTransactions(const TArray<FOnlinePurchaseTransactionIOS>& Transactions)
{
	for (const FOnlinePurchaseTransactionIOS& Transaction : Transactions)
	{
		int32 FinalizingTransactionIndex = FinalizingTransactions.IndexOfByPredicate(
			[&Transaction](const FKnownTransaction& KnownTransaction) 
			{
				return KnownTransaction.Receipt->TransactionId == Transaction.GetTransactionIdentifier();
			});
		
		if (FinalizingTransactionIndex != INDEX_NONE)
		{
			FinalizingTransactions.RemoveAt(FinalizingTransactionIndex);
			OnTransactionFinalized.Broadcast(Transaction);
		}
	}
}

void FOnlinePurchaseIOS::AddReceiptToCache(TArray<FKnownTransaction>& Cache, const TSharedRef<FPurchaseReceipt>& Receipt, const FOnlinePurchaseTransactionIOS& Transaction)
{
    FKnownTransaction* CachedTransaction = Cache.FindByPredicate([&Transaction](const FKnownTransaction& KnownTransaction) {
        return KnownTransaction.Receipt->TransactionId == Transaction.GetTransactionIdentifier();
    });
    if(CachedTransaction)
    {
		CachedTransaction->Receipt = Receipt;
    }
    else
    {
		Cache.Emplace(Receipt, Transaction.GetPaymentTransaction());
    }
}

void FOnlinePurchaseIOS::OnQueryReceiptsComplete(bool bSuccess, const TSharedPtr<TArray<FOnlinePurchaseTransactionIOS>>& KnownTransactions)
{
	TArray< FKnownTransaction > IntermediateCachedReceipts;
	
	if (KnownTransactions)
	{
		for (const FOnlinePurchaseTransactionIOS& Transaction: *KnownTransactions)
		{
			const EPurchaseTransactionState TransactionState = Transaction.GetTransactionState();
			if (TransactionState == EPurchaseTransactionState::Restored || TransactionState == EPurchaseTransactionState::Purchased)
			{
				TSharedRef<FPurchaseReceipt> Receipt = GenerateOfflineReceipt(Transaction);
				AddReceiptToCache(IntermediateCachedReceipts, Receipt, Transaction);
			}
		}
	}
	// Full restore is complete
	bQueryingReceipts = false;
	if (bSuccess)
	{
		CachedReceipts = MoveTemp(IntermediateCachedReceipts);
	}
	Subsystem->ExecuteNextTick([this, bSuccess]() {
		FOnlineError FinalResult(bSuccess);
		QueryReceiptsComplete.ExecuteIfBound(FinalResult);
		QueryReceiptsComplete.Unbind();
	});
}

void FOnlinePurchaseIOS::FinalizeReceiptValidationInfo(const FUniqueNetId& UserId, FString& InReceiptValidationInfo, const FOnFinalizeReceiptValidationInfoComplete& Delegate)
{
	FOnlineError DefaultSuccess(true);
	Delegate.ExecuteIfBound(DefaultSuccess, InReceiptValidationInfo);
}

TSharedRef<FPurchaseReceipt> FOnlinePurchaseIOS::GenerateFailReceipt(const FPurchaseCheckoutRequest& CheckoutRequest)
{
    TSharedRef<FPurchaseReceipt> Receipt = MakeShared<FPurchaseReceipt>();
    
    Receipt->TransactionState = EPurchaseTransactionState::Failed;
    
    // Add the requested offers to the receipt in the event of an incomplete purchase.
    for(const auto& RequestedOffer : CheckoutRequest.PurchaseOffers)
    {
        Receipt->AddReceiptOffer(RequestedOffer.OfferNamespace, RequestedOffer.OfferId, RequestedOffer.Quantity);
    }
    return Receipt;
}

TSharedRef<FPurchaseReceipt> FOnlinePurchaseIOS::GenerateOfflineReceipt(const FOnlinePurchaseTransactionIOS& Transaction)
{
	const EPurchaseTransactionState TransactionState = Transaction.GetTransactionState();
    TSharedRef<FPurchaseReceipt> Receipt = MakeShared<FPurchaseReceipt>();

	Receipt->TransactionState = TransactionState;
	Receipt->TransactionId = Transaction.GetTransactionIdentifier();
	
	if (TransactionState == EPurchaseTransactionState::Purchased ||
		TransactionState == EPurchaseTransactionState::Restored)
	{
		FPurchaseReceipt::FReceiptOfferEntry ReceiptEntry(TEXT(""), Transaction.GetOfferId(), Transaction.GetQuantity());
		
		FPurchaseReceipt::FLineItemInfo& LineItem = ReceiptEntry.LineItems.Emplace_GetRef();
		
		LineItem.ItemName = Transaction.GetOfferId();
		LineItem.UniqueId = Transaction.GetTransactionIdentifier();
		LineItem.ValidationInfo = Transaction.GetReceiptData();

		Receipt->AddReceiptOffer(ReceiptEntry);
	}
	
	return Receipt;
}

TSharedRef<FPurchaseReceipt> FOnlinePurchaseIOS::GenerateReceipt(const FPurchaseCheckoutRequest& CheckoutRequest, const FOnlinePurchaseTransactionIOS& Transaction)
{
	const EPurchaseTransactionState TransactionState = Transaction.GetTransactionState();
    TSharedRef<FPurchaseReceipt> Receipt = GenerateOfflineReceipt(Transaction);

	if(TransactionState != EPurchaseTransactionState::Purchased &&
	   TransactionState != EPurchaseTransactionState::Restored)
	{
        // Add the requested offers to the receipt in the event of an incomplete purchase.
        for(const auto& RequestedOffer : CheckoutRequest.PurchaseOffers)
        {
            Receipt->AddReceiptOffer(RequestedOffer.OfferNamespace, RequestedOffer.OfferId, RequestedOffer.Quantity);
        }
	}
	
	return Receipt;
}

#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
void FOnlinePurchaseIOS::RequestExternalPurchaseToken(const FOnRequestExternalPurchaseTokenComplete& Delegate) const
{
	FMarketplaceKitModule::RequestCTTokenAsync(Delegate);
}
#endif // !PLATFORM_TVOS && !PLATFORM_VISIONOS

#undef LOCTEXT_NAMESPACE

#endif //UE_WITH_STORE_KIT
