// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSessionInterfaceIOS.h"

#include "Apple/AppleStringUtils.h"
#include "IOS/IOSAsyncTask.h"

#include "OnlineSessionSettings.h"
#include "OnlineSubsystemIOS.h"

#if !PLATFORM_TVOS // @todo tvos: What is up with all this being busted?? Multipeer is gone, but so is older stuff? What's to do??

@implementation FGameCenterSessionDelegateMC
@synthesize Session;
@synthesize PeerID;

- (NSString *)fixDisplayName:(NSString *)displayName
{
	// initWithDisplayName: expects the display name to have a maximum length of 63 bytes in UTF-8 encoding, not be nil nor empty
	static constexpr NSUInteger MaxValidDisplayNameSize = 63;

	if (displayName != nil)
	{
		if ([displayName lengthOfBytesUsingEncoding: NSUTF8StringEncoding] > MaxValidDisplayNameSize)
		{
			char Buffer[MaxValidDisplayNameSize] = {0};
			NSUInteger UsedLength = 0;
			NSRange Range = NSMakeRange(0, [displayName length]);
			[displayName getBytes: Buffer maxLength: MaxValidDisplayNameSize usedLength: &UsedLength encoding: NSUTF8StringEncoding options: 0 range: Range remainingRange: NULL];
			if (NSString *FixedDisplayName = [[NSString alloc] initWithBytes: Buffer length: UsedLength encoding: NSUTF8StringEncoding])
			{
				return FixedDisplayName;
			}
		}
		else if (displayName.length > 0)
		{
			return displayName;
		}
	}
	return @"Unknown";
}

- (instancetype)initSessionWithName:(NSString*) sessionName andDisplayName:(NSString*) displayName
{
    UE_LOG_ONLINE_SESSION(Display, TEXT("- (void)initSessionWithName:(NSString*) sessionName"));
	self = [super init];
	NSString* FixedDisplayName = [self fixDisplayName: displayName];
	self.PeerID = [[[MCPeerID alloc] initWithDisplayName: FixedDisplayName] autorelease];
    self.Session = [[[MCSession alloc] initWithPeer:self.PeerID] autorelease];
	self.Session.delegate = self;
	return self;
}

-(void)dealloc
{
	[Session release];
	[PeerID release];
	[super dealloc];
}

- (void)shutdownSession
{
    UE_LOG_ONLINE_SESSION(Display, TEXT("- (void)shutdownSession"));
    [self.Session disconnect];
    self.Session.delegate = nil;
}

- (bool)sessionsAvailable
{
    return NO;
}

-(void)joinSession
{
}

- (void)session:(MCSession *)session didReceiveData:(NSData *)data fromPeer:(MCPeerID *)peerID
{
    
}

- (void)session:(MCSession *)session didStartReceivingResourceWithName:(NSString *)resourceName fromPeer:(MCPeerID *)peerID withProgress:(NSProgress *)progress
{
}

- (void)session:(MCSession *)session didFinishReceivingResourceWithName:(NSString *)resourceName fromPeer:(MCPeerID *)peerID atURL:(NSURL *)localURL withError:(NSError *)error
{
    
}

- (void)session:(MCSession *)session didReceiveStream:(NSInputStream *)stream withName:(NSString *)streamName fromPeer:(MCPeerID *)peerID
{
    
}

- (void)session:(MCSession *)session peer:(MCPeerID *)peerID didChangeState:(MCSessionState)state
{
    const FString PeerId(PeerID.displayName);
    switch (state)
    {
        case MCSessionStateConnected:
        {
            UE_LOG_ONLINE_SESSION(Display, TEXT("Peer connected: %s"), *PeerId);
            break;
        }
            
        case MCSessionStateNotConnected:
        {
            UE_LOG_ONLINE_SESSION(Display, TEXT("Peer not connected: %s"), *PeerId);
            break;
        }
    }
}

@end

@implementation FGameCenterSessionDelegate
@synthesize SessionMC;

- (instancetype)initSessionWithName:(NSString*) sessionName andDisplayName: (NSString*)displayName
{
    UE_LOG_ONLINE_SESSION(Display, TEXT("- (void)initSessionWithName:(NSString*) sessionName (NSString*)displayName"));
	self = [super init];
	// Create the session object
    if ([MCSession class])
    {
        self.SessionMC = [[[FGameCenterSessionDelegateMC alloc] initSessionWithName:sessionName andDisplayName: displayName] autorelease];
    }
    
    return self;
}

-(void)dealloc
{
	[SessionMC release];
	[super dealloc];
}

- (void)shutdownSession
{
    UE_LOG_ONLINE_SESSION(Display, TEXT("- (void)shutdownSession"));
    if ([MCSession class])
    {
        [self.SessionMC shutdownSession];
    }
}

- (bool)sessionsAvailable
{
    UE_LOG_ONLINE_SESSION(Display, TEXT("- (void)shutdownSession"));
    if ([MCSession class])
    {
        return [self.SessionMC sessionsAvailable];
    }
    return NO;
}

- (void)joinSession
{
    UE_LOG_ONLINE_SESSION(Display, TEXT("- (void)shutdownSession"));
    if ([MCSession class])
    {
        [self.SessionMC joinSession];
    }
}

@end



#endif


FOnlineSessionIOS::FOnlineSessionIOS() :
	IOSSubsystem(NULL),
	CurrentSessionSearch(NULL)
{
	UE_LOG_ONLINE_SESSION(Display, TEXT("FOnlineSessionIOS::FOnlineSessionIOS()"));
}


FOnlineSessionIOS::FOnlineSessionIOS(FOnlineSubsystemIOS* InSubsystem) :
	IOSSubsystem(InSubsystem),
	CurrentSessionSearch(NULL)
{
	UE_LOG_ONLINE_SESSION(Display, TEXT("FOnlineSessionIOS::FOnlineSessionIOS(FOnlineSubsystemIOS* InSubsystem)"));
}


FOnlineSessionIOS::~FOnlineSessionIOS()
{

}


void FOnlineSessionIOS::Tick(float DeltaTime)
{

}


FNamedOnlineSession* FOnlineSessionIOS::AddNamedSession(FName SessionName, const FOnlineSessionSettings& SessionSettings)
{
	FScopeLock ScopeLock(&SessionLock);
	return new (Sessions) FNamedOnlineSession(SessionName, SessionSettings);
}


FNamedOnlineSession* FOnlineSessionIOS::AddNamedSession(FName SessionName, const FOnlineSession& Session)
{
	FScopeLock ScopeLock(&SessionLock);
	return new (Sessions) FNamedOnlineSession(SessionName, Session);
}


FUniqueNetIdPtr FOnlineSessionIOS::CreateSessionIdFromString(const FString& SessionIdStr)
{
	ensureMsgf(false, TEXT("NYI"));
	FUniqueNetIdPtr SessionId;
	return SessionId;
}

FNamedOnlineSession* FOnlineSessionIOS::GetNamedSession(FName SessionName)
{
	FScopeLock ScopeLock(&SessionLock);
	for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
	{
		if (Sessions[SearchIndex].SessionName == SessionName)
		{
			return &Sessions[SearchIndex];
		}
	}
	return NULL;
}


void FOnlineSessionIOS::RemoveNamedSession(FName SessionName)
{
	FScopeLock ScopeLock(&SessionLock);
	for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
	{
		if (Sessions[SearchIndex].SessionName == SessionName)
		{
			Sessions.RemoveAtSwap(SearchIndex);
			return;
		}
	}
}


EOnlineSessionState::Type FOnlineSessionIOS::GetSessionState(FName SessionName) const
{
	FScopeLock ScopeLock(&SessionLock);
	for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
	{
		if (Sessions[SearchIndex].SessionName == SessionName)
		{
			return Sessions[SearchIndex].SessionState;
		}
	}

	return EOnlineSessionState::NoSession;
}


bool FOnlineSessionIOS::HasPresenceSession()
{
	FScopeLock ScopeLock(&SessionLock);
	for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
	{
		if (Sessions[SearchIndex].SessionSettings.bUsesPresence)
		{
			return true;
		}
	}
		
	return false;
}


bool FOnlineSessionIOS::CreateSession(int32 HostingPlayerNum, FName SessionName, const FOnlineSessionSettings& NewSessionSettings)
{
	bool bSuccessfullyCreatedSession = false;
#if !PLATFORM_TVOS
	UE_LOG_ONLINE_SESSION(Display, TEXT("FOnlineSessionIOS::CreateSession"));
	
	// Check for an existing session
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session == NULL)
	{
		Session = AddNamedSession(SessionName, NewSessionSettings);
		Session->SessionState = EOnlineSessionState::Pending;
		UE_LOG_ONLINE_SESSION(Display, TEXT("Creating new session."));

		// Create the session object
        FGameCenterSessionDelegate* NewGKSession = [FGameCenterSessionDelegate alloc];
		if( NewGKSession != NULL )
		{
			UE_LOG_ONLINE_SESSION(Display, TEXT("Created session delegate"));
			NSString* SafeSessionName = [NSString stringWithFString:SessionName.ToString()];
			GKSessions.Add(SessionName, NewGKSession);
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to create session delegate"));
		}
		
		bSuccessfullyCreatedSession = true;
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Display, TEXT("Cannot create session '%s': session already exists."), *SessionName.ToString());
	}
	
	UE_LOG_ONLINE_SESSION(Display, TEXT("TriggerOnCreateSessionCompleteDelegates: %s, %d"), *SessionName.ToString(), bSuccessfullyCreatedSession);
	TriggerOnCreateSessionCompleteDelegates(SessionName, bSuccessfullyCreatedSession);
#endif
	
	return bSuccessfullyCreatedSession;
}


bool FOnlineSessionIOS::CreateSession(const FUniqueNetId& HostingPlayerId, FName SessionName, const FOnlineSessionSettings& NewSessionSettings)
{
	// HostingPlayerNum is unused, can pass in anything
	return CreateSession(0, SessionName, NewSessionSettings);
}


bool FOnlineSessionIOS::StartSession(FName SessionName)
{
	bool bSuccessfullyStartedSession = false;
#if !PLATFORM_TVOS
	UE_LOG_ONLINE_SESSION(Display, TEXT("FOnlineSessionIOS::StartSession"));
		
	// Check for an existing session
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session != NULL)
	{
		// Find the linked GK session and start it.
        FGameCenterSessionDelegate* LinkedGKSession = *GKSessions.Find( SessionName );
		NSString* SafeSessionName = [NSString stringWithFString:SessionName.ToString()];

		IOnlineIdentityPtr IdentityInterface = IOSSubsystem->GetIdentityInterface();
		NSString* Nickname = IdentityInterface->GetPlayerNickname(0).GetNSString();

		[LinkedGKSession initSessionWithName:SafeSessionName andDisplayName: Nickname];

		// Update the session state as we are now running.
		Session->SessionState = EOnlineSessionState::InProgress;
		bSuccessfullyStartedSession = true;
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to create session delegate"));
	}

	TriggerOnStartSessionCompleteDelegates(SessionName, bSuccessfullyStartedSession);
#endif
	return bSuccessfullyStartedSession;
}


bool FOnlineSessionIOS::UpdateSession(FName SessionName, FOnlineSessionSettings& UpdatedSessionSettings, bool bShouldRefreshOnlineData)
{
	bool bSuccessfullyUpdatedSession = false;
	
	UE_LOG_ONLINE_SESSION(Display, TEXT("FOnlineSessionIOS::UpdateSession - not implemented"));

	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		Session->SessionSettings = UpdatedSessionSettings;

		bSuccessfullyUpdatedSession = true;
	}

	TriggerOnUpdateSessionCompleteDelegates(SessionName, bSuccessfullyUpdatedSession);
	
	return bSuccessfullyUpdatedSession;
}


bool FOnlineSessionIOS::EndSession(FName SessionName)
{
	bool bSuccessfullyEndedSession = false;
	
	UE_LOG_ONLINE_SESSION(Display, TEXT("FOnlineSessionIOS::EndSession - not implemented"));

	TriggerOnEndSessionCompleteDelegates(SessionName, bSuccessfullyEndedSession);

	return bSuccessfullyEndedSession;
}


bool FOnlineSessionIOS::DestroySession(FName SessionName, const FOnDestroySessionCompleteDelegate& CompletionDelegate)
{
	bool bSuccessfullyDestroyedSession = false;
#if !PLATFORM_TVOS
	
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session != NULL)
	{
		FGameCenterSessionDelegate* ExistingGKSession = *GKSessions.Find( SessionName );
		[ExistingGKSession shutdownSession];
		
		// The session info is no longer needed
		RemoveNamedSession( Session->SessionName );

		GKSessions.Remove( SessionName );

		bSuccessfullyDestroyedSession = true;
	}

	CompletionDelegate.ExecuteIfBound(SessionName, bSuccessfullyDestroyedSession);
	TriggerOnDestroySessionCompleteDelegates(SessionName, bSuccessfullyDestroyedSession);
#endif
	return bSuccessfullyDestroyedSession;
}

bool FOnlineSessionIOS::IsPlayerInSession(FName SessionName, const FUniqueNetId& UniqueId)
{
	return IsPlayerInSessionImpl(this, SessionName, UniqueId);
}

bool FOnlineSessionIOS::StartMatchmaking(const TArray< FUniqueNetIdRef >& LocalPlayers, FName SessionName, const FOnlineSessionSettings& NewSessionSettings, TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	UE_LOG_ONLINE_SESSION(Warning, TEXT("StartMatchmaking is not supported on this platform. Use FindSessions or FindSessionById."));
	TriggerOnMatchmakingCompleteDelegates(SessionName, false);
	return false;
}


bool FOnlineSessionIOS::CancelMatchmaking(int32 SearchingPlayerNum, FName SessionName)
{
	UE_LOG_ONLINE_SESSION(Warning, TEXT("CancelMatchmaking is not supported on this platform. Use CancelFindSessions."));
	TriggerOnCancelMatchmakingCompleteDelegates(SessionName, false);
	return false;
}


bool FOnlineSessionIOS::CancelMatchmaking(const FUniqueNetId& SearchingPlayerId, FName SessionName)
{
	UE_LOG_ONLINE_SESSION(Warning, TEXT("CancelMatchmaking is not supported on this platform. Use CancelFindSessions."));
	TriggerOnCancelMatchmakingCompleteDelegates(SessionName, false);
	return false;
}

bool FOnlineSessionIOS::FindSessions(int32 SearchingPlayerNum, const TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	bool bSuccessfullyFoundSessions = false;

#if !PLATFORM_TVOS
	// Don't start another search while one is in progress
	if (!CurrentSessionSearch.IsValid() && SearchSettings->SearchState != EOnlineAsyncTaskState::InProgress)
	{
		for (TMap< FName, FGameCenterSessionDelegate* >::TConstIterator SessionIt(GKSessions); SessionIt; ++SessionIt)
		{
			FGameCenterSessionDelegate* GKSession = SessionIt.Value();
            bSuccessfullyFoundSessions = [GKSession sessionsAvailable];
		}
	}

	TriggerOnFindSessionsCompleteDelegates(bSuccessfullyFoundSessions);
#endif
	return bSuccessfullyFoundSessions;
}


bool FOnlineSessionIOS::FindSessions(const FUniqueNetId& SearchingPlayerId, const TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	// SearchingPlayerNum is unused, can pass in anything
	return FindSessions(0, SearchSettings);
}

bool FOnlineSessionIOS::FindSessionById(const FUniqueNetId& SearchingUserId, const FUniqueNetId& SessionId, const FUniqueNetId& FriendId, const FOnSingleSessionResultCompleteDelegate& CompletionDelegates)
{
	FOnlineSessionSearchResult EmptyResult;
	CompletionDelegates.ExecuteIfBound(0, false, EmptyResult);
	return true;
}

bool FOnlineSessionIOS::CancelFindSessions()
{
	bool bSuccessfullyCancelledSession = false;

	UE_LOG_ONLINE_SESSION(Display, TEXT("FOnlineSessionIOS::CancelSession - not implemented"));
	
	TriggerOnCancelFindSessionsCompleteDelegates(true);

	return bSuccessfullyCancelledSession;
}


bool FOnlineSessionIOS::PingSearchResults(const FOnlineSessionSearchResult& SearchResult)
{
	bool bSuccessfullyPingedSearchResults = false;

	UE_LOG_ONLINE_SESSION(Display, TEXT("FOnlineSessionIOS::PingSearchResults - not implemented"));
	
	return bSuccessfullyPingedSearchResults;
}


bool FOnlineSessionIOS::JoinSession(int32 PlayerNum, FName SessionName, const FOnlineSessionSearchResult& DesiredSession)
{
	EOnJoinSessionCompleteResult::Type JoinSessionResult = EOnJoinSessionCompleteResult::UnknownError;
#if !PLATFORM_TVOS
	UE_LOG_ONLINE_SESSION(Display, TEXT("FOnlineSessionIOS::JoinSession"));

	FGameCenterSessionDelegate* SessionDelegate = *GKSessions.Find( SessionName );
	if( SessionDelegate != NULL )
	{
        [SessionDelegate joinSession];
		JoinSessionResult = EOnJoinSessionCompleteResult::Success;
	}

	TriggerOnJoinSessionCompleteDelegates(SessionName, JoinSessionResult);
#endif
	return JoinSessionResult == EOnJoinSessionCompleteResult::Success;
}


bool FOnlineSessionIOS::JoinSession(const FUniqueNetId& PlayerId, FName SessionName, const FOnlineSessionSearchResult& DesiredSession)
{
	// PlayerNum is unused, can pass in anything
	return JoinSession(0, SessionName, DesiredSession);
}


bool FOnlineSessionIOS::FindFriendSession(int32 LocalUserNum, const FUniqueNetId& Friend)
{
	if (LocalUserNum == PLATFORMUSERID_NONE)
	{
		TArray<FOnlineSessionSearchResult> EmptyResult;
		TriggerOnFindFriendSessionCompleteDelegates(LocalUserNum, false, EmptyResult);
		return false;
	}

	return FindFriendSession(*IOSSubsystem->GetIdentityInterface()->GetUniquePlayerId(LocalUserNum).ToSharedRef(), Friend);
}


bool FOnlineSessionIOS::FindFriendSession(const FUniqueNetId& LocalUserId, const FUniqueNetId& Friend)
{
	TArray<FUniqueNetIdRef> FriendList;
	FriendList.Add(Friend.AsShared());
	return FindFriendSession(LocalUserId, FriendList);
}

bool FOnlineSessionIOS::FindFriendSession(const FUniqueNetId& LocalUserId, const TArray<FUniqueNetIdRef>& FriendList)
{
	bool bSuccessfullyJoinedFriendSession = false;

	UE_LOG_ONLINE_SESSION(Display, TEXT("FOnlineSessionIOS::FindFriendSession - not implemented"));

	int32 LocalUserNum = IOSSubsystem->GetIdentityInterface()->GetPlatformUserIdFromUniqueNetId(LocalUserId);

	TArray<FOnlineSessionSearchResult> EmptyResult;
	TriggerOnFindFriendSessionCompleteDelegates(LocalUserNum, bSuccessfullyJoinedFriendSession, EmptyResult);

	return bSuccessfullyJoinedFriendSession;
}

bool FOnlineSessionIOS::SendSessionInviteToFriend(int32 LocalUserNum, FName SessionName, const FUniqueNetId& Friend)
{
	bool bSuccessfullySentSessionInviteToFriend = false;
	
	UE_LOG_ONLINE_SESSION(Display, TEXT("FOnlineSessionIOS::SendSessionInviteToFriend - not implemented"));
	
	return bSuccessfullySentSessionInviteToFriend;
}


bool FOnlineSessionIOS::SendSessionInviteToFriend(const FUniqueNetId& LocalUserId, FName SessionName, const FUniqueNetId& Friend)
{
	UE_LOG_ONLINE_SESSION(Display, TEXT("FOnlineSessionIOS::SendSessionInviteToFriend - not implemented"));
	
	return false;
}


bool FOnlineSessionIOS::SendSessionInviteToFriends(int32 LocalUserNum, FName SessionName, const TArray< FUniqueNetIdRef >& Friends)
{
	bool bSuccessfullySentSessionInviteToFriends = false;

	UE_LOG_ONLINE_SESSION(Display, TEXT("FOnlineSessionIOS::SendSessionInviteToFriends - not implemented"));
	
	return bSuccessfullySentSessionInviteToFriends;
}


bool FOnlineSessionIOS::SendSessionInviteToFriends(const FUniqueNetId& LocalUserId, FName SessionName, const TArray< FUniqueNetIdRef >& Friends)
{
	UE_LOG_ONLINE_SESSION(Display, TEXT("FOnlineSessionIOS::SendSessionInviteToFriends - not implemented"));
	
	return false;
}


bool FOnlineSessionIOS::GetResolvedConnectString(FName SessionName, FString& ConnectInfo, FName PortType)
{
	bool bSuccessfullyGotResolvedConnectString = false;
	
	UE_LOG_ONLINE_SESSION(Display, TEXT("FOnlineSessionIOS::GetResolvedConnectString - not implemented"));
	
	return bSuccessfullyGotResolvedConnectString;
}


bool FOnlineSessionIOS::GetResolvedConnectString(const FOnlineSessionSearchResult& SearchResult, FName PortType, FString& ConnectInfo)
{
	return false;
}


FOnlineSessionSettings* FOnlineSessionIOS::GetSessionSettings(FName SessionName)
{
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		return &Session->SessionSettings;
	}
	return NULL;
}


bool FOnlineSessionIOS::RegisterPlayer(FName SessionName, const FUniqueNetId& PlayerId, bool bWasInvited)
{
	bool bSuccessfullyRegisteredPlayer = false;
	
	UE_LOG_ONLINE_SESSION(Display, TEXT("FOnlineSessionIOS::RegisterPlayer - not implemented"));	
	
	TArray< FUniqueNetIdRef > Players;
	Players.Add(PlayerId.AsShared());
	
	bSuccessfullyRegisteredPlayer = RegisterPlayers(SessionName, Players, bWasInvited);

	return bSuccessfullyRegisteredPlayer;
}


bool FOnlineSessionIOS::RegisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players, bool bWasInvited)
{
	bool bSuccessfullyRegisteredPlayers = false;
	
	UE_LOG_ONLINE_SESSION(Display, TEXT("FOnlineSessionIOS::RegisterPlayers - not implemented"));
	
	for (int32 PlayerIdx=0; PlayerIdx < Players.Num(); PlayerIdx++)
	{
		TriggerOnRegisterPlayersCompleteDelegates(SessionName, Players, bSuccessfullyRegisteredPlayers);
	}

	return bSuccessfullyRegisteredPlayers;
}


bool FOnlineSessionIOS::UnregisterPlayer(FName SessionName, const FUniqueNetId& PlayerId)
{
	bool bSuccessfullyUnregisteredPlayer = false;
	
	UE_LOG_ONLINE_SESSION(Display, TEXT("FOnlineSessionIOS::UnregisterPlayer - not implemented"));
	
	TArray< FUniqueNetIdRef > Players;
	Players.Add(PlayerId.AsShared());
	bSuccessfullyUnregisteredPlayer = UnregisterPlayers(SessionName, Players);
	
	return bSuccessfullyUnregisteredPlayer;
}


bool FOnlineSessionIOS::UnregisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players)
{
	bool bSuccessfullyUnregisteredPlayers = false;

	UE_LOG_ONLINE_SESSION(Display, TEXT("FOnlineSessionIOS::UnregisterPlayers - not implemented"));

	for (int32 PlayerIdx=0; PlayerIdx < Players.Num(); PlayerIdx++)
	{
		TriggerOnUnregisterPlayersCompleteDelegates(SessionName, Players, bSuccessfullyUnregisteredPlayers);
	}

	return bSuccessfullyUnregisteredPlayers;
}


int32 FOnlineSessionIOS::GetNumSessions()
{
	FScopeLock ScopeLock(&SessionLock);
	return Sessions.Num();
}


void FOnlineSessionIOS::DumpSessionState()
{

}

void FOnlineSessionIOS::RegisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnRegisterLocalPlayerCompleteDelegate& Delegate)
{
	Delegate.ExecuteIfBound(PlayerId, EOnJoinSessionCompleteResult::Success);
}

void FOnlineSessionIOS::UnregisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnUnregisterLocalPlayerCompleteDelegate& Delegate)
{
	Delegate.ExecuteIfBound(PlayerId, true);
}
