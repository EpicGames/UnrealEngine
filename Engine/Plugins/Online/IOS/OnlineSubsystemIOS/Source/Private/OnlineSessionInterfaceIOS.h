// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#import <UIKit/UIKit.h>
#import <GameKit/GKLocalPlayer.h>

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystemIOSTypes.h"

#if !PLATFORM_TVOS

#include <MultipeerConnectivity/MultipeerConnectivity.h>

@interface FGameCenterSessionDelegateMC : UIViewController<MCSessionDelegate>
{
};

@property (nonatomic, strong) MCPeerID *PeerID;
@property (nonatomic, strong) MCSession *Session;

-(instancetype) initSessionWithName:(NSString*) sessionName andDisplayName:(NSString*) displayName;
-(void) shutdownSession;
-(bool) sessionsAvailable;
-(void) joinSession;

@end

@interface FGameCenterSessionDelegate : NSObject
{
};

@property (nonatomic, strong) FGameCenterSessionDelegateMC *SessionMC;

-(instancetype) initSessionWithName:(NSString*) sessionName andDisplayName:(NSString*) displayName;
-(void) shutdownSession;
-(bool) sessionsAvailable;
-(void) joinSession;

@end

#endif


/**
 * Interface definition for the online services session services 
 * Session services are defined as anything related managing a session 
 * and its state within a platform service
 */
class FOnlineSessionIOS : public IOnlineSession
{
private:

	/** Reference to the main GameCenter subsystem */
	class FOnlineSubsystemIOS* IOSSubsystem;

	/** Hidden on purpose */
	FOnlineSessionIOS();

PACKAGE_SCOPE:

	/** Critical sections for thread safe operation of session lists */
	mutable FCriticalSection SessionLock;

	/** Current session settings */
	TArray<FNamedOnlineSession> Sessions;

#if !PLATFORM_TVOS
	TMap< FName, FGameCenterSessionDelegate* > GKSessions;
#endif

	/** Current search object */
	TSharedPtr<FOnlineSessionSearch> CurrentSessionSearch;


PACKAGE_SCOPE:

	/** Constructor */
	FOnlineSessionIOS(class FOnlineSubsystemIOS* InSubsystem);

	/**
	 * Session tick for various background tasks
	 */
	void Tick(float DeltaTime);


	//~ Begin IOnlineSession Interface
	class FNamedOnlineSession* AddNamedSession(FName SessionName, const FOnlineSessionSettings& SessionSettings) override;

	class FNamedOnlineSession* AddNamedSession(FName SessionName, const FOnlineSession& Session) override;

public:

	virtual ~FOnlineSessionIOS();

	virtual FUniqueNetIdPtr CreateSessionIdFromString(const FString& SessionIdStr) override;

	FNamedOnlineSession* GetNamedSession(FName SessionName) override;

	virtual void RemoveNamedSession(FName SessionName) override;

	virtual EOnlineSessionState::Type GetSessionState(FName SessionName) const override;

	virtual bool HasPresenceSession() override;

	virtual bool CreateSession(int32 HostingPlayerNum, FName SessionName, const FOnlineSessionSettings& NewSessionSettings) override;

	virtual bool CreateSession(const FUniqueNetId& HostingPlayerId, FName SessionName, const FOnlineSessionSettings& NewSessionSettings) override;

	virtual bool StartSession(FName SessionName) override;

	virtual bool UpdateSession(FName SessionName, FOnlineSessionSettings& UpdatedSessionSettings, bool bShouldRefreshOnlineData = false) override;

	virtual bool EndSession(FName SessionName) override;

	virtual bool DestroySession(FName SessionName, const FOnDestroySessionCompleteDelegate& CompletionDelegate = FOnDestroySessionCompleteDelegate()) override;

	virtual bool IsPlayerInSession(FName SessionName, const FUniqueNetId& UniqueId) override;

	virtual bool StartMatchmaking(const TArray< FUniqueNetIdRef >& LocalPlayers, FName SessionName, const FOnlineSessionSettings& NewSessionSettings, TSharedRef<FOnlineSessionSearch>& SearchSettings) override;

	virtual bool CancelMatchmaking(int32 SearchingPlayerNum, FName SessionName) override;

	virtual bool CancelMatchmaking(const FUniqueNetId& SearchingPlayerId, FName SessionName) override;

	virtual bool FindSessions(int32 SearchingPlayerNum, const TSharedRef<FOnlineSessionSearch>& SearchSettings) override;

	virtual bool FindSessions(const FUniqueNetId& SearchingPlayerId, const TSharedRef<FOnlineSessionSearch>& SearchSettings) override;

	virtual bool FindSessionById(const FUniqueNetId& SearchingUserId, const FUniqueNetId& SessionId, const FUniqueNetId& FriendId, const FOnSingleSessionResultCompleteDelegate& CompletionDelegate) override;

	virtual bool CancelFindSessions() override;

	virtual bool PingSearchResults(const FOnlineSessionSearchResult& SearchResult) override;

	virtual bool JoinSession(int32 PlayerNum, FName SessionName, const FOnlineSessionSearchResult& DesiredSession) override;

	virtual bool JoinSession(const FUniqueNetId& PlayerId, FName SessionName, const FOnlineSessionSearchResult& DesiredSession) override;

	virtual bool FindFriendSession(int32 LocalUserNum, const FUniqueNetId& Friend) override;

	virtual bool FindFriendSession(const FUniqueNetId& LocalUserId, const FUniqueNetId& Friend) override;

	virtual bool FindFriendSession(const FUniqueNetId& LocalUserId, const TArray<FUniqueNetIdRef>& FriendList) override;

	virtual bool SendSessionInviteToFriend(int32 LocalUserNum, FName SessionName, const FUniqueNetId& Friend) override;

	virtual bool SendSessionInviteToFriend(const FUniqueNetId& LocalUserId, FName SessionName, const FUniqueNetId& Friend) override;

	virtual bool SendSessionInviteToFriends(int32 LocalUserNum, FName SessionName, const TArray< FUniqueNetIdRef >& Friends) override;

	virtual bool SendSessionInviteToFriends(const FUniqueNetId& LocalUserId, FName SessionName, const TArray< FUniqueNetIdRef >& Friends) override;

	virtual bool GetResolvedConnectString(FName SessionName, FString& ConnectInfo, FName PortType) override;

	virtual bool GetResolvedConnectString(const class FOnlineSessionSearchResult& SearchResult, FName PortType, FString& ConnectInfo) override;

	virtual FOnlineSessionSettings* GetSessionSettings(FName SessionName) override;

	virtual bool RegisterPlayer(FName SessionName, const FUniqueNetId& PlayerId, bool bWasInvited) override;

	virtual bool RegisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players, bool bWasInvited = false) override;

	virtual bool UnregisterPlayer(FName SessionName, const FUniqueNetId& PlayerId) override;

	virtual bool UnregisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players) override;

	virtual void RegisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnRegisterLocalPlayerCompleteDelegate& Delegate) override;

	virtual void UnregisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnUnregisterLocalPlayerCompleteDelegate& Delegate) override;

	virtual int32 GetNumSessions() override;

	virtual void DumpSessionState() override;
	//~ End IOnlineSession Interface
};

typedef TSharedPtr<FOnlineSessionIOS, ESPMode::ThreadSafe> FOnlineSessionIOSPtr;

