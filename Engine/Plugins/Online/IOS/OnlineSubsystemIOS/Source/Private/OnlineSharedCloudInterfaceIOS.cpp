// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSharedCloudInterfaceIOS.h"
#include "Misc/ScopeLock.h"

//
// SharedCloud interface implementation
//

FOnlineSharedCloudInterfaceIOS::~FOnlineSharedCloudInterfaceIOS()
{
	ClearSharedFiles();
}

FCloudFile* FOnlineSharedCloudInterfaceIOS::GetCloudFile(const FString& FileName, bool bCreateIfMissing)
{
    FScopeLock ScopeLock(&CloudDataLock);
	if (FileName.Len() > 0)
	{
		for (int32 FileIdx = 0; FileIdx < CloudFileData.Num(); FileIdx++)
		{
			FCloudFile* UserFileData = &CloudFileData[FileIdx];
			if (UserFileData &&
				UserFileData->FileName == FileName)
			{
				return UserFileData;
			}
		}

		if (bCreateIfMissing)
		{
			return new (CloudFileData)FCloudFile(FileName);
		}
	}

	return NULL;
}

bool FOnlineSharedCloudInterfaceIOS::ClearFiles()
{
    FScopeLock ScopeLock(&CloudDataLock);
	// Delete file contents
	for (int32 FileIdx = 0; FileIdx < CloudFileData.Num(); FileIdx++)
	{
		CloudFileData[FileIdx].Data.Empty();
	}

	// No async files being handled, so empty them all
	CloudFileData.Empty();
	return true;
}

bool FOnlineSharedCloudInterfaceIOS::ClearCloudFile(const FString& FileName)
{
    FScopeLock ScopeLock(&CloudDataLock);
	int32 FoundIndex = INDEX_NONE;
	for (int32 FileIdx = 0; FileIdx < CloudFileData.Num(); FileIdx++)
	{
		FCloudFile* UserFileData = &CloudFileData[FileIdx];
		if (UserFileData->FileName == FileName)
		{
			// If there is an async task outstanding, fail to empty
			if (UserFileData->AsyncState == EOnlineAsyncTaskState::InProgress)
			{
				return false;
			}

			UserFileData->Data.Empty();
			FoundIndex = FileIdx;
			break;
		}
	}

	if (FoundIndex != INDEX_NONE)
	{
		CloudFileData.RemoveAtSwap(FoundIndex);
	}
	return true;
}

bool FOnlineSharedCloudInterfaceIOS::GetSharedFileContents(const FSharedContentHandle& SharedHandle, TArray<uint8>& FileContents)
{
	FCloudFile* CloudFile = GetCloudFile(SharedHandle.ToString());
	if (CloudFile && CloudFile->AsyncState == EOnlineAsyncTaskState::Done && CloudFile->Data.Num() > 0)
	{
		FileContents = CloudFile->Data;
		return true;
	}
	return false;
}

bool FOnlineSharedCloudInterfaceIOS::ClearSharedFiles()
{
	// NOTE: Return true regardless of if the user was valid or not. We don't care if
	// there weren't any files in need of clearing, only if there was a failure to clear
	ClearFiles();
	return true;
}

bool FOnlineSharedCloudInterfaceIOS::ClearSharedFile(const FSharedContentHandle& SharedHandle)
{
	// NOTE: Return true regardless of if the file exists or not. The only way to return false
	// would be in we failed to delete, and not deleting a non-existing file is not a failure
	ClearCloudFile(SharedHandle.ToString());
	return true;
}

bool FOnlineSharedCloudInterfaceIOS::ReadSharedFile(const FSharedContentHandle& SharedHandle)
{
	FCloudFile* CloudFile = GetCloudFile(SharedHandle.ToString());
	if (CloudFile)
	{
		CloudFile->AsyncState = EOnlineAsyncTaskState::InProgress;
        __block FSharedContentHandleIOS NewHandle = *((FSharedContentHandleIOS*)&SharedHandle);
		return [[IOSCloudStorage cloudStorage] readFile:SharedHandle.ToString().GetNSString() sharedDB : true completionHandler : ^ (CKRecord *record, NSError *error)
		{
            FScopeLock ScopeLock(&CloudDataLock);
            FCloudFile* File = GetCloudFile(NewHandle.ToString());
			if (error)
			{
				// TODO: record is potentially not found
				File->AsyncState = EOnlineAsyncTaskState::Failed;
				TriggerOnReadSharedFileCompleteDelegates(false, NewHandle);
				NSLog(@"Error: %@", error);
			}
			else
			{
				// store the contents in the memory record database
				NSData* data = (NSData*)record[@"contents"];
				File->Data.Empty();
				File->Data.Append((uint8*)data.bytes, data.length);
				File->AsyncState = EOnlineAsyncTaskState::Done;
				TriggerOnReadSharedFileCompleteDelegates(true, NewHandle);
				NSLog(@"Record Read!");
			}
		}];
	}
    else
    {
        TriggerOnReadSharedFileCompleteDelegates(false, SharedHandle);
    }
	return false;
}

bool FOnlineSharedCloudInterfaceIOS::WriteSharedFile(const FUniqueNetId& UserId, const FString& FileName, TArray<uint8>& FileContents)
{
	FCloudFile* CloudFile = GetCloudFile(FileName, true);
	if (CloudFile)
	{
		CloudFile->AsyncState = EOnlineAsyncTaskState::InProgress;
        __block FString NewFile = FileName;
        __block TArray<uint8> DataContents = FileContents;
		return [[IOSCloudStorage cloudStorage] writeFile:FileName.GetNSString() contents : [[NSData alloc] initWithBytes:FileContents.GetData() length : FileContents.Num()] sharedDB : true completionHandler : ^ (CKRecord *record, NSError *error)
		{
            FScopeLock ScopeLock(&CloudDataLock);
            FCloudFile* File = GetCloudFile(NewFile);
			if (error)
			{
				// TODO: record is potentially newer on the server
				File->AsyncState = EOnlineAsyncTaskState::Failed;
				TSharedRef<FSharedContentHandle> SharedHandle = MakeShareable(new FSharedContentHandleIOS());
				TriggerOnWriteSharedFileCompleteDelegates(false, UserId, NewFile, SharedHandle);
				NSLog(@"Error: %@", error);
			}
			else
			{
				File->Data = DataContents;
				File->AsyncState = EOnlineAsyncTaskState::Done;
				TSharedRef<FSharedContentHandle> SharedHandle = MakeShareable(new FSharedContentHandleIOS(NewFile));
				TriggerOnWriteSharedFileCompleteDelegates(true, UserId, NewFile, SharedHandle);
				NSLog(@"Record Saved!");
			}
		}];
	}
    else
    {
        TSharedRef<FSharedContentHandle> SharedHandle = MakeShareable(new FSharedContentHandleIOS());
        TriggerOnWriteSharedFileCompleteDelegates(false, UserId, FileName, SharedHandle);
    }
	return false;
}

void FOnlineSharedCloudInterfaceIOS::GetDummySharedHandlesForTest(TArray< TSharedRef<FSharedContentHandle> >& OutHandles)
{
    OutHandles.Add(MakeShareable(new FSharedContentHandleIOS(TEXT("TestData1"))));
    OutHandles.Add(MakeShareable(new FSharedContentHandleIOS(TEXT("TestData2"))));
    OutHandles.Add(MakeShareable(new FSharedContentHandleIOS(TEXT("TestData3"))));
    OutHandles.Add(MakeShareable(new FSharedContentHandleIOS(TEXT("TestData4"))));
    OutHandles.Add(MakeShareable(new FSharedContentHandleIOS(TEXT("TestData5"))));
}