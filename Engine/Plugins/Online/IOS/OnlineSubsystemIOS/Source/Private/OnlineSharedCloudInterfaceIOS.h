// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "Interfaces/OnlineSharedCloudInterface.h"
#include "OnlineUserCloudInterfaceIOS.h"


/**
* iOS specific implementation of a shared file handle
*/
class FSharedContentHandleIOS :
	public FSharedContentHandle
{

	/** Holds the handle to the shared content */
	const FString SharedContentHandle;

public:

	FSharedContentHandleIOS()
		: SharedContentHandle()
	{
	}

	/**
	* Constructs this object with the specified shared content id
	*
	* @param InUniqueNetId the id to set ours to
	*/
	FSharedContentHandleIOS(const FString& InSharedContentHandle) :
		SharedContentHandle(InSharedContentHandle)
	{
	}

	/**
	*	Comparison operator
	*/
	bool operator==(const FSharedContentHandleIOS& Other) const
	{
		return (SharedContentHandle == Other.SharedContentHandle);
	}

	/**
	* Get the raw byte representation of this shared content handle
	* This data is platform dependent and shouldn't be manipulated directly
	*
	* @return byte array of size GetSize()
	*/
	virtual const uint8* GetBytes() const override
	{
		return (uint8*)(&SharedContentHandle);
	}

	/**
	* Get the size of this shared content handle
	*
	* @return size in bytes of the id representation
	*/
	virtual int32 GetSize() const override
	{
		return SharedContentHandle.GetAllocatedSize();
	}

	/**
	* Check the validity of this shared content handle
	*
	* @return true if this is a well formed ID, false otherwise
	*/
	virtual bool IsValid() const override
	{
		return SharedContentHandle != FString();
	}

	/**
	* Platform specific conversion to string representation of data
	*
	* @return data in string form
	*/
	virtual FString ToString() const override
	{
		return SharedContentHandle;
	}

	/**
	* Get a human readable representation of this shared content handle
	* Shouldn't be used for anything other than logging/debugging
	*
	* @return handle in string form
	*/
	virtual FString ToDebugString() const override
	{
		return SharedContentHandle;
	}
};

/**
 * Provides the interface for sharing files already on the cloud with other users
 */
class FOnlineSharedCloudInterfaceIOS : public IOnlineSharedCloud
{
protected:

	FCloudFile* GetCloudFile(const FString& FileName, bool bCreateIfMissing = false);
	bool ClearFiles();
	bool ClearCloudFile(const FString& FileName);

PACKAGE_SCOPE:

	FOnlineSharedCloudInterfaceIOS()
	{
	}

public:

	virtual ~FOnlineSharedCloudInterfaceIOS();

	// IOnlineSharedCloud
	virtual bool GetSharedFileContents(const FSharedContentHandle& SharedHandle, TArray<uint8>& FileContents) override;
	virtual bool ClearSharedFiles() override;
	virtual bool ClearSharedFile(const FSharedContentHandle& SharedHandle) override;
	virtual bool ReadSharedFile(const FSharedContentHandle& SharedHandle) override;
	virtual bool WriteSharedFile(const FUniqueNetId& UserId, const FString& Filename, TArray<uint8>& FileContents) override;
	virtual void GetDummySharedHandlesForTest(TArray< TSharedRef<FSharedContentHandle> > & OutHandles) override;

private:
	/** File cache */
	TArray<struct FCloudFile> CloudFileData;
    
    /** Critical section for thread safe operation on cloud files */
    FCriticalSection CloudDataLock;
};

typedef TSharedPtr<FOnlineSharedCloudInterfaceIOS, ESPMode::ThreadSafe> FOnlineSharedCloudIOSPtr;
