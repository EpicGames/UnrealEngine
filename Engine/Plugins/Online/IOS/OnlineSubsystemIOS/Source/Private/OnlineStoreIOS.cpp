// Copyright Epic Games, Inc. All Rights Reserved.

#if UE_WITH_STORE_KIT

#include "OnlineStoreIOS.h"
#include "Apple/AppleStringUtils.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Culture.h"
#include "Internationalization/FastDecimalFormat.h"
#include "OnlineSubsystem.h"
#import <StoreKit/SKProduct.h>
#import <StoreKit/SKProductsRequest.h>
#include "Misc/ConfigCacheIni.h"

#include "OnlineSubsystemIOS-Swift.h"
#include "StoreKitSwiftInterop.h"
#include "IOSAppDelegate.h"
#include "IOS/IOSAsyncTask.h"

#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
#include "MarketplaceKit.h"
#endif

@interface FSKProductsRequestHelper : SKProductsRequest
{
};

/** Delegate to fire when this product request completes with the store kit */
@property FOnQueryOnlineStoreOffersComplete OfferDelegate;
@end

@implementation FSKProductsRequestHelper
@end

/**
 * Proxy class that notifies updates from FSKProductsRequestHelper to FOnlineStoreIOS on the game 
 * thread (SKRequest responses are received in worker threads)
 */
@interface FStoreKitStoreProxy : NSObject<SKProductsRequestDelegate>
{
    FOnlineStoreIOS* _StoreReceiver;
    NSMutableSet* _Requests;
};
@end

@implementation FStoreKitStoreProxy

////////////////////////////////////////////////////////////////////
/// SKProductsRequestDelegate implementation

-(void)productsRequest: (SKProductsRequest *)Request didReceiveResponse: (SKProductsResponse *)Response
{
    UE_LOG_ONLINE_STOREV2(Log, TEXT("FStoreKitStoreProxy::didReceiveResponse"));

    // Response for SKRequest is received in a working thread. Use a task to notify from game thread
    FSKProductsRequestHelper* Helper = (FSKProductsRequestHelper*)Request;
    [FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
    {
        if(_StoreReceiver)
        {
            _StoreReceiver->OnProductsRequestResponse(Response, Helper.OfferDelegate);
        }
        return true;
    }];
}

- (void)request:(SKRequest *)Request didFailWithError:(NSError *)Error
{
    UE_LOG_ONLINE_STOREV2(Log, TEXT("FStoreKitStoreProxy::didFailWithError"));

    // Response for SKRequest is received in a working thread. Use a task to notify from game thread
    FSKProductsRequestHelper* Helper = (FSKProductsRequestHelper*)Request;
    [FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
    {
        if(_StoreReceiver)
        {
            _StoreReceiver->OnProductsRequestResponse(nil, Helper.OfferDelegate);
        }
        return true;
    }];}

-(void)requestDidFinish:(SKRequest*)Request
{
    [_Requests removeObject:Request];
}

////////////////////////////////////////////////////////////////////
/// FStoreKitStoreProxy methods

- (id)initWithReceiver: (FOnlineStoreIOS*)StoreReceiver
{
    _StoreReceiver = StoreReceiver;
    _Requests = [[NSMutableSet alloc] init];
    return self;
}

-(void)dealloc
{
    for(FSKProductsRequestHelper* Request in _Requests)
    {
        [Request cancel];
    }
    [_Requests release];
    
    [super dealloc];
}

-(void)Shutdown
{
    _StoreReceiver = nullptr;
}

-(void)requestProductData: (NSMutableSet*)ProductIDs WithDelegate : (const FOnQueryOnlineStoreOffersComplete&)Delegate
{
    UE_LOG_ONLINE_STOREV2(Log, TEXT("FStoreKitStoreProxy::requestProductData"));

    FSKProductsRequestHelper* Request = [[[FSKProductsRequestHelper alloc] initWithProductIdentifiers:ProductIDs] autorelease];
    Request.delegate = self;
    Request.OfferDelegate = Delegate;
    [_Requests addObject:Request];
    [Request start];
}

@end

////////////////////////////////////////////////////////////////////
/// FOnlineStoreIOS implementation

FOnlineStoreIOS::FOnlineStoreIOS(FOnlineSubsystemIOS* InSubsystem)
	: bIsQueryInFlight(false)
	, Subsystem(InSubsystem)
{
	UE_LOG_ONLINE_STOREV2(Verbose, TEXT( "FOnlineStoreIOS::FOnlineStoreIOS" ));
    StoreKitProxy = [[[FStoreKitStoreProxy alloc] initWithReceiver:this] autorelease];
}

FOnlineStoreIOS::~FOnlineStoreIOS()
{
    [StoreKitProxy Shutdown];
}

void FOnlineStoreIOS::QueryCategories(const FUniqueNetId& UserId, const FOnQueryOnlineStoreCategoriesComplete& Delegate)
{
	Delegate.ExecuteIfBound(false, TEXT("No CatalogService"));
}

void FOnlineStoreIOS::GetCategories(TArray<FOnlineStoreCategory>& OutCategories) const
{
	OutCategories.Empty();
}

void FOnlineStoreIOS::QueryOffersByFilter(const FUniqueNetId& UserId, const FOnlineStoreFilter& Filter, const FOnQueryOnlineStoreOffersComplete& Delegate)
{
	Delegate.ExecuteIfBound(false, TArray<FUniqueOfferId>(), TEXT("No CatalogService"));
}

bool FOnlineStoreIOS::OffersNotAllowedInLocale(const FString& InLocale)
{
    UE_LOG_ONLINE_STOREV2(Log, TEXT("Locale: %s"), *InLocale);
    // get the data from the config file
    TArray<FString> BannedLocales;
    GConfig->GetArray(TEXT("OnlineSubsystemIOS.Store"), TEXT("BannedLocales"), BannedLocales, GEngineIni);
    if (BannedLocales.Num() == 0)
    {
        // no banned locales just let the offer proceed
        return false;
    }
    
    TArray<FString> LocaleData;
    InLocale.ParseIntoArray(LocaleData, TEXT("-"));
    FString Locale = LocaleData.Num() > 1 ? LocaleData[1] : LocaleData[0];
    return BannedLocales.Contains(Locale);
}

void FOnlineStoreIOS::QueryOffersById(const FUniqueNetId& UserId, const TArray<FUniqueOfferId>& OfferIds, const FOnQueryOnlineStoreOffersComplete& Delegate)
{
	UE_LOG_ONLINE_STOREV2(Verbose, TEXT("FOnlineStoreIOS::QueryOffersById"));
	
	if (bIsQueryInFlight)
	{
		Delegate.ExecuteIfBound(false, OfferIds, TEXT("Request already in flight"));
	}
	else if (OfferIds.Num() == 0)
	{
		Delegate.ExecuteIfBound(false, OfferIds, TEXT("No offers to query for"));
	}
    else if (OffersNotAllowedInLocale(FPlatformMisc::GetDefaultLocale()))
    {
        TArray<FUniqueOfferId> OfferedIds;
        Delegate.ExecuteIfBound(true, OfferedIds, TEXT(""));
    }
	else
	{
		// autoreleased NSSet to hold IDs
		NSMutableSet* ProductSet = [NSMutableSet setWithCapacity:OfferIds.Num()];
		for (int32 OfferIdx = 0; OfferIdx < OfferIds.Num(); OfferIdx++)
		{
			NSString* ID = [NSString stringWithFString:OfferIds[OfferIdx]];
			// convert to NSString for the set objects
			[ProductSet addObject:ID];
		}
		
        [StoreKitProxy requestProductData:ProductSet WithDelegate:Delegate];		
		bIsQueryInFlight = true;
	}
}

/**
 * Convert an Apple SKProduct reference into a engine FOnlineStoreOffer
 * (Apple has only Title/Description converted to Title/(short)Description)
 *
 * @param Product information about an Apple offer from iTunesConnect
 *
 * @return FOnlineStoreOffer with proper parameters filled in
 */
TSharedPtr<FOnlineStoreOffer> ConvertProductToStoreOffer(SKProduct* Product)
{
    TSharedPtr<FOnlineStoreOffer> NewProductInfo = MakeShared<FOnlineStoreOffer>();
	
	NewProductInfo->OfferId = [Product productIdentifier];
	
	NewProductInfo->Title = FText::FromString([Product localizedTitle]);
	NewProductInfo->Description = FText::FromString([Product localizedDescription]);
	//NewProductInfo->LongDescription = FText::FromString([Product localizedDescription]);
	NewProductInfo->CurrencyCode = [Product.priceLocale objectForKey : NSLocaleCurrencyCode];
	
	// Convert the backend stated price into its base units
	FInternationalization& I18N = FInternationalization::Get();
	const FCulture& Culture = *I18N.GetCurrentCulture();
	
	const FDecimalNumberFormattingRules& FormattingRules = Culture.GetCurrencyFormattingRules(NewProductInfo->CurrencyCode);
	const FNumberFormattingOptions& FormattingOptions = FormattingRules.CultureDefaultFormattingOptions;
	double Val = static_cast<double>([Product.price doubleValue]) * static_cast<double>(FMath::Pow(10.0f, FormattingOptions.MaximumFractionalDigits));
	NewProductInfo->NumericPrice = FMath::TruncToInt(Val + 0.5);
	
	// iOS doesn't support these fields, set to min and max defaults
	NewProductInfo->ReleaseDate = FDateTime::MinValue();
	NewProductInfo->ExpirationDate = FDateTime::MaxValue();

	NewProductInfo->PriceText = FText::AsCurrencyBase(NewProductInfo->NumericPrice, NewProductInfo->CurrencyCode);
	
	return NewProductInfo;
}

void FOnlineStoreIOS::OnProductsRequestResponse(SKProductsResponse* Response, const FOnQueryOnlineStoreOffersComplete& CompletionDelegate)
{
    if(bIsQueryInFlight)
    {
        TArray<FUniqueOfferId> OfferIds;
        bool bWasSuccessful = (Response != nil);
        if(bWasSuccessful)
        {
            for (SKProduct* Product in Response.products)
            {
                FOnlineStoreOfferIOS NewProductOffer(Product, ConvertProductToStoreOffer(Product));
                
                AddOffer(NewProductOffer);
                OfferIds.Add(NewProductOffer.Offer->OfferId);
                
                UE_LOG_ONLINE_STOREV2(Log, TEXT("Product Identifier: %s, Name: %s, Desc: %s, Long Desc: %s, Price: %s IntPrice: %lld"),
                    *NewProductOffer.Offer->OfferId,
                    *NewProductOffer.Offer->Title.ToString(),
                    *NewProductOffer.Offer->Description.ToString(),
                    *NewProductOffer.Offer->LongDescription.ToString(),
                    *NewProductOffer.Offer->PriceText.ToString(),
                    NewProductOffer.Offer->NumericPrice);
            }
            
            for (NSString *invalidProduct in Response.invalidProductIdentifiers)
            {
                UE_LOG_ONLINE_STOREV2(Warning, TEXT("Problem in iTunes connect configuration for product: %s"), *FString(invalidProduct));
            }
        }        
        CompletionDelegate.ExecuteIfBound(bWasSuccessful, OfferIds, TEXT(""));
        bIsQueryInFlight = false;
    }
}

void FOnlineStoreIOS::AddOffer(const FOnlineStoreOfferIOS& NewOffer)
{
	if (NewOffer.IsValid())
	{
        FOnlineStoreOfferIOS* Existing = CachedOffers.Find(NewOffer.Offer->OfferId);
        if (Existing != nullptr)
        {
            *Existing = NewOffer;
        }
        else
        {
            CachedOffers.Add(NewOffer.Offer->OfferId, NewOffer);
        }
    }
}

void FOnlineStoreIOS::GetOffers(TArray<FOnlineStoreOfferRef>& OutOffers) const
{
	for (const auto& CachedEntry : CachedOffers)
	{
		const FOnlineStoreOfferIOS& CachedOffer = CachedEntry.Value;
        OutOffers.Add(CachedOffer.Offer.ToSharedRef());
	}
}

TSharedPtr<FOnlineStoreOffer> FOnlineStoreIOS::GetOffer(const FUniqueOfferId& OfferId) const
{
	TSharedPtr<FOnlineStoreOffer> Result;

	const FOnlineStoreOfferIOS* Existing = CachedOffers.Find(OfferId);
	if (Existing != nullptr)
	{
		Result = Existing->Offer;
	}
	
	return Result;
}

SKProduct* FOnlineStoreIOS::GetSKProductByOfferId(const FUniqueOfferId& OfferId)
{
	const FOnlineStoreOfferIOS* Existing = CachedOffers.Find(OfferId);
	if (Existing != nullptr)
	{
		return Existing->Product;
	}
	
	return nil;
}

void FOnlineStoreIOS::QueryStoreCountry()
{
	TWeakPtr<FOnlineStoreIOS> WeakThis = AsWeak();
	[StoreKit2 getCountry: ^ (NSString* _Nonnull Country)
	{
		if (TSharedPtr<FOnlineStoreIOS> SharedThis = WeakThis.Pin())
		{
			const FString CountryString(Country);
			SharedThis->StoreCountry = CountryString;
			SharedThis->bIsStoreCountryInitialized.Store(true);
			
			UE_LOG_ONLINE_STOREV2(Verbose, TEXT("Obtained store country: %s"), *CountryString);
		}
	}];
}

FString FOnlineStoreIOS::GetStoreCountry() const
{
	if (bIsStoreCountryInitialized.Load())
	{
		return StoreCountry;
	}
	
	UE_LOG_ONLINE_STOREV2(Warning, TEXT("Attempting to get store country before it is initialized!"));
	return "";
}

#if !UE_BUILD_SHIPPING
class FStoreExec : public FSelfRegisteringExec
{
	virtual bool Exec_Dev(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		if (!FParse::Command(&Cmd, TEXT("ASC")))
		{
			return false;
		}

		if (FParse::Command(&Cmd, TEXT("REFUND")))
		{
			TCHAR TransIdStr[256];
			if (FParse::Token(Cmd, TransIdStr, UE_ARRAY_COUNT(TransIdStr), true))
			{
				uint64 TransactionId = FCString::Atoi64(TransIdStr);
				[StoreKit2 testRefund:TransactionId inScene:[IOSAppDelegate GetDelegate].window.windowScene];
			}

			return true;
		}
		
		return false;
	}
	
} GStoreExec;

#endif

#endif //UE_WITH_STORE_KIT
