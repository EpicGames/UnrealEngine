// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemIOS.h"

#include "IOS/IOSAppDelegate.h"
#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
#include "MarketplaceKit.h"
#endif
#include "Misc/ConfigCacheIni.h"
#import "OnlineAppStoreUtils.h"
#include "Stats/Stats.h"

FOnlineSubsystemIOS::FOnlineSubsystemIOS(FName InInstanceName)
	: FOnlineSubsystemImpl(IOS_SUBSYSTEM, InInstanceName)
	, AppStoreHelper(nil)
{
}

IOnlineSessionPtr FOnlineSubsystemIOS::GetSessionInterface() const
{
	return SessionInterface;
}

IOnlineFriendsPtr FOnlineSubsystemIOS::GetFriendsInterface() const
{
	return FriendsInterface;
}

IOnlinePartyPtr FOnlineSubsystemIOS::GetPartyInterface() const
{
	return nullptr;
}

IOnlineGroupsPtr FOnlineSubsystemIOS::GetGroupsInterface() const
{
	return nullptr;
}

IOnlineSharedCloudPtr FOnlineSubsystemIOS::GetSharedCloudInterface() const
{
	return SharedCloudInterface;
}

IOnlineUserCloudPtr FOnlineSubsystemIOS::GetUserCloudInterface() const
{
	return UserCloudInterface;
}

IOnlineLeaderboardsPtr FOnlineSubsystemIOS::GetLeaderboardsInterface() const
{
	return LeaderboardsInterface;
}

IOnlineVoicePtr FOnlineSubsystemIOS::GetVoiceInterface() const
{
	return nullptr;
}

IOnlineExternalUIPtr FOnlineSubsystemIOS::GetExternalUIInterface() const
{
	return ExternalUIInterface;
}

IOnlineTimePtr FOnlineSubsystemIOS::GetTimeInterface() const
{
	return nullptr;
}

IOnlineIdentityPtr FOnlineSubsystemIOS::GetIdentityInterface() const
{
	return IdentityInterface;
}

IOnlineTitleFilePtr FOnlineSubsystemIOS::GetTitleFileInterface() const
{
	return nullptr;
}

IOnlineEntitlementsPtr FOnlineSubsystemIOS::GetEntitlementsInterface() const
{
	return nullptr;
}

IOnlineStoreV2Ptr FOnlineSubsystemIOS::GetStoreV2Interface() const
{
#if UE_WITH_STORE_KIT
	return StoreV2Interface;
#else
	return nullptr;
#endif
}

IOnlinePurchasePtr FOnlineSubsystemIOS::GetPurchaseInterface() const
{
#if UE_WITH_STORE_KIT
	return PurchaseInterface;
#else
	return nullptr;
#endif
}

IOnlineEventsPtr FOnlineSubsystemIOS::GetEventsInterface() const
{
	return nullptr;
}

IOnlineAchievementsPtr FOnlineSubsystemIOS::GetAchievementsInterface() const
{
	return AchievementsInterface;
}

IOnlineSharingPtr FOnlineSubsystemIOS::GetSharingInterface() const
{
	return nullptr;
}

IOnlineUserPtr FOnlineSubsystemIOS::GetUserInterface() const
{
	return nullptr;
}

IOnlineMessagePtr FOnlineSubsystemIOS::GetMessageInterface() const
{
	return nullptr;
}

IOnlinePresencePtr FOnlineSubsystemIOS::GetPresenceInterface() const
{
	return nullptr;
}

IOnlineChatPtr FOnlineSubsystemIOS::GetChatInterface() const
{
	return nullptr;
}

IOnlineStatsPtr FOnlineSubsystemIOS::GetStatsInterface() const
{
	return nullptr;
}

IOnlineTurnBasedPtr FOnlineSubsystemIOS::GetTurnBasedInterface() const
{
	return TurnBasedInterface;
}

IOnlineTournamentPtr FOnlineSubsystemIOS::GetTournamentInterface() const
{
	return nullptr;
}

bool FOnlineSubsystemIOS::Init() 
{
	bool bSuccessfullyStartedUp = true;
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineSubsystemIOS::Init()"));
	
	if( !IsEnabled() )
	{
		UE_LOG_ONLINE(Warning, TEXT("All iOS online features have been disabled in the system settings"));
		bSuccessfullyStartedUp = false;
	}
	else
	{
		SessionInterface = MakeShareable(new FOnlineSessionIOS(this));
		IdentityInterface = MakeShareable(new FOnlineIdentityIOS(this));

		if (IsGameCenterEnabled())
		{
			FriendsInterface = MakeShareable(new FOnlineFriendsIOS(this));
			LeaderboardsInterface = MakeShareable(new FOnlineLeaderboardsIOS(this));
			AchievementsInterface = MakeShareable(new FOnlineAchievementsIOS(this));
			ExternalUIInterface = MakeShareable(new FOnlineExternalUIIOS(this));
			TurnBasedInterface = MakeShareable(new FOnlineTurnBasedIOS());
		}

		UserCloudInterface = MakeShareable(new FOnlineUserCloudInterfaceIOS());
		SharedCloudInterface = MakeShareable(new FOnlineSharedCloudInterfaceIOS());

#if UE_WITH_STORE_KIT
		if (IsInAppPurchasingEnabled())
		{
#if !PLATFORM_TVOS && !PLATFORM_VISIONOS		// tvOS & VisionOS ONLY support AppStore
			EMarketplaceType CurrentMarketplace = FMarketplaceKitModule::GetCurrentTypeStatic();
			if (CurrentMarketplace == EMarketplaceType::AppStore || 
				CurrentMarketplace == EMarketplaceType::TestFlight ||
				CurrentMarketplace == EMarketplaceType::Marketplace)
#endif
			{
				StoreV2Interface = MakeShareable(new FOnlineStoreIOS(this));
				PurchaseInterface = MakeShareable(new FOnlinePurchaseIOS(this));
			}
		}
#endif
		if (UserCloudInterface && IsCloudKitEnabled())
		{
			FString IOSCloudKitSyncStrategy = "";
			GConfig->GetString(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("IOSCloudKitSyncStrategy"), IOSCloudKitSyncStrategy, GEngineIni);

			if (!IOSCloudKitSyncStrategy.Equals("None"))
			{
				UserCloudInterface->InitCloudSave(IOSCloudKitSyncStrategy.Equals("Always"));
			}
		}

		InitAppStoreHelper();
		
		if (StoreV2Interface)
		{
			// Initialize the store country
			StoreV2Interface->QueryStoreCountry();
		}
	}

	return bSuccessfullyStartedUp;
}

void FOnlineSubsystemIOS::InitAppStoreHelper()
{
	AppStoreHelper = [[FAppStoreUtils alloc] init];
}

void FOnlineSubsystemIOS::CleanupAppStoreHelper()
{
	[AppStoreHelper release];
	AppStoreHelper = nil;
}

FAppStoreUtils* FOnlineSubsystemIOS::GetAppStoreUtils()
{
	return AppStoreHelper;
}

bool FOnlineSubsystemIOS::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSubsystemIOS_Tick);

	if (!FOnlineSubsystemImpl::Tick(DeltaTime))
	{
		return false;
	}

	if (SessionInterface.IsValid())
	{
		SessionInterface->Tick(DeltaTime);
	}
	return true;
}

FText FOnlineSubsystemIOS::GetOnlineServiceName() const
{
	return NSLOCTEXT("OnlineSubsystemIOS", "OnlineServiceName", "Game Center");
}

bool FOnlineSubsystemIOS::Shutdown() 
{
	bool bSuccessfullyShutdown = true;
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineSubsystemIOS::Shutdown()"));

	bSuccessfullyShutdown = FOnlineSubsystemImpl::Shutdown();
	
#define DESTRUCT_INTERFACE(Interface) \
	if (Interface.IsValid()) \
	{ \
		UE_LOG_ONLINE(Display, TEXT(#Interface));\
		ensure(Interface.IsUnique()); \
		Interface = nullptr; \
	}
	
	DESTRUCT_INTERFACE(SessionInterface);
	DESTRUCT_INTERFACE(IdentityInterface);
	DESTRUCT_INTERFACE(FriendsInterface);
	DESTRUCT_INTERFACE(LeaderboardsInterface);
	DESTRUCT_INTERFACE(AchievementsInterface);
	DESTRUCT_INTERFACE(ExternalUIInterface);
	DESTRUCT_INTERFACE(TurnBasedInterface);
	DESTRUCT_INTERFACE(UserCloudInterface);
	DESTRUCT_INTERFACE(SharedCloudInterface);
#if UE_WITH_STORE_KIT
	DESTRUCT_INTERFACE(StoreV2Interface);
	DESTRUCT_INTERFACE(PurchaseInterface);
#endif

#undef DESTRUCT_INTERFACE
	
	// Cleanup after the interfaces are free
	CleanupAppStoreHelper();
	
	return bSuccessfullyShutdown;
}

FString FOnlineSubsystemIOS::GetAppId() const 
{
	return TEXT( "" );
}

bool FOnlineSubsystemIOS::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)  
{
	bool bWasHandled = false;
	if (FOnlineSubsystemImpl::Exec(InWorld, Cmd, Ar))
	{
		bWasHandled = true;
	}
	else
	{ 
		if (FParse::Command(&Cmd, TEXT("PURCHASE")))
		{
			bWasHandled = HandlePurchaseExecCommands(InWorld, Cmd, Ar);
		}
	}

	return bWasHandled;
}

bool FOnlineSubsystemIOS::HandlePurchaseExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = false;

#if UE_WITH_STORE_KIT
	if (FParse::Command(&Cmd, TEXT("DUMPAPPRECEIPT")))
	{
        PurchaseInterface->DumpAppReceipt();
		bWasHandled = true;
	}
#endif
	return bWasHandled;
}

bool FOnlineSubsystemIOS::IsEnabled() const
{
	bool bEnableGameCenter = IsGameCenterEnabled();

	const bool bEnableCloudKit = IsCloudKitEnabled();

	const bool bIsInAppPurchasingEnabled = IsInAppPurchasingEnabled();
	const bool bIsEnabledByConfig = FOnlineSubsystemImpl::IsEnabled(); // TODO: Do we want to enable this by this config?
	
	return bEnableGameCenter || bEnableCloudKit || bIsInAppPurchasingEnabled || bIsEnabledByConfig;
}

bool FOnlineSubsystemIOS::IsGameCenterEnabled()
{
	bool bEnableGameCenter = false;
	GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bEnableGameCenterSupport"), bEnableGameCenter, GEngineIni);

	return bEnableGameCenter;
}

bool FOnlineSubsystemIOS::IsCloudKitEnabled()
{
	bool bEnableCloudKit;
	GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bEnableCloudKitSupport"), bEnableCloudKit, GEngineIni);

	return bEnableCloudKit;
}

bool FOnlineSubsystemIOS::IsInAppPurchasingEnabled()
{
	bool bEnableIAP = false;
	GConfig->GetBool(TEXT("OnlineSubsystemIOS.Store"), TEXT("bSupportsInAppPurchasing"), bEnableIAP, GEngineIni);
	
	bool bEnableIAP1 = false;
	if (GConfig->GetBool(TEXT("OnlineSubsystemIOS.Store"), TEXT("bSupportInAppPurchasing"), bEnableIAP1, GEngineIni))
	{
		UE_LOG_ONLINE(Warning, TEXT("Deprecated use of bSupportInAppPurchasing setting in OnlineSubsystemIOS.Store. Use bSupportsInAppPurchasing instead"));
	}
	return UE_WITH_STORE_KIT && (bEnableIAP || bEnableIAP1);
}

NSString* FOnlineSubsystemIOS::GetPlayerId(GKPlayer* Player)
{
	return Player.gamePlayerID;
}

NSString* FOnlineSubsystemIOS::GetPlayerId(GKLocalPlayer* Player)
{
	return Player.gamePlayerID;
}

