// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemIOSModule.h"
#include "HttpModule.h"
#include "OnlineSubsystem.h"
#include "OnlineSubsystemIOS.h"

IMPLEMENT_MODULE( FOnlineSubsystemIOSModule, OnlineSubsystemIOS );

/**
 * Class responsible for creating instance(s) of the subsystem
 */
class FOnlineFactoryIOS : public IOnlineFactory
{

private:

	/** Single instantiation of the IOS interface */
	static FOnlineSubsystemIOSPtr IOSSingleton;

	virtual void DestroySubsystem()
	{
		if (IOSSingleton.IsValid())
		{
			IOSSingleton->Shutdown();
			IOSSingleton = NULL;
		}
	}

public:

	FOnlineFactoryIOS() {}
	virtual ~FOnlineFactoryIOS() 
	{
		DestroySubsystem();
	}

	virtual IOnlineSubsystemPtr CreateSubsystem(FName InstanceName)
	{
		if (!IOSSingleton.IsValid())
		{
			IOSSingleton = MakeShared<FOnlineSubsystemIOS, ESPMode::ThreadSafe>(InstanceName);
			if (IOSSingleton->IsEnabled())
			{
				if(!IOSSingleton->Init())
				{
					UE_LOG_ONLINE(Warning, TEXT("FOnlineSubsystemIOSModule failed to initialize!"));
					DestroySubsystem();
				}
			}
			else
			{
				UE_LOG_ONLINE(Warning, TEXT("FOnlineSubsystemIOSModule was disabled"));
				DestroySubsystem();
			}

			return IOSSingleton;
		}

		UE_LOG_ONLINE(Warning, TEXT("Can't create more than one instance of IOS online subsystem!"));
		return NULL;
	}
};

FOnlineSubsystemIOSPtr FOnlineFactoryIOS::IOSSingleton = NULL;

void FOnlineSubsystemIOSModule::StartupModule()
{
	SCOPED_BOOT_TIMING("FOnlineSubsystemIOSModule::StartupModule");

	UE_LOG_ONLINE(Display, TEXT("FOnlineSubsystemIOSModule::StartupModule()"));

	FHttpModule::Get();

	IOSFactory = new FOnlineFactoryIOS();

	// Create and register our singleton factory with the main online subsystem for easy access
	FOnlineSubsystemModule& OSS = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>("OnlineSubsystem");
	OSS.RegisterPlatformService(IOS_SUBSYSTEM, IOSFactory);
}


void FOnlineSubsystemIOSModule::ShutdownModule()
{
	UE_LOG_ONLINE(Display, TEXT("FOnlineSubsystemIOSModule::ShutdownModule()"));

	delete IOSFactory;
	IOSFactory = NULL;
}
