// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemTypes.h"
#include "OnlineSubsystemIOSPackage.h"

class FOnlineSubsystemIOS;

// from OnlineSubsystemTypes.h
TEMP_UNIQUENETIDSTRING_SUBCLASS(FUniqueNetIdIOS, IOS_SUBSYSTEM);

