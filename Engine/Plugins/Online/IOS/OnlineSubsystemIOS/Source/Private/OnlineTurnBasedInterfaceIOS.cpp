// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineTurnBasedInterfaceIOS.h"
#include "TurnBasedEventListener.h"
#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystem.h"
#include "OnlineSubsystemIOS.h"
#include "OnlineAsyncTaskManager.h"
#include "Net/RepLayout.h"
#include "Interfaces/TurnBasedMatchInterface.h"
#include <GameKit/GKLocalPlayer.h>

#include "IOS/IOSAsyncTask.h"

DEFINE_LOG_CATEGORY_STATIC(LogTurnBasedInterfaceIOS, Verbose, All);

#define INCLUDE_LOCAL_PLAYER (true)
#define DO_NOT_INCLUDE_LOCAL_PLAYER (false)

FTurnBasedMatchIOS::FTurnBasedMatchIOS(GKTurnBasedMatch* _Match, NSArray* PlayerArray)
	: FTurnBasedMatch()
	, Match([_Match retain])
{
	if (!Match)
	{
		UE_LOGF(LogTurnBasedInterfaceIOS, Error, "GKTurnBasedMatch required to create a FTurnBasedMatchIOS");
	}

	for (GKPlayer* player in PlayerArray)
	{
		PlayerAliasArray.Add(UTF8_TO_TCHAR(player.displayName.UTF8String));
	}
}

FTurnBasedMatchIOS::~FTurnBasedMatchIOS()
{
	[Match release];
}

int32 FTurnBasedMatchIOS::GetNumberOfPlayers() const
{
	return Match.participants.count;
}

bool FTurnBasedMatchIOS::GetPlayerDisplayName(int32 PlayerIndex, FString& Name) const
{
	if (PlayerIndex >= PlayerAliasArray.Num() || PlayerIndex < 0)
	{
		return FTurnBasedMatch::GetPlayerDisplayName(PlayerIndex, Name);
	}

	Name = PlayerAliasArray[PlayerIndex];
	return true;
}

void FTurnBasedMatchIOS::ReloadMatchData(FDownloadMatchDataSignature DownloadCallback)
{
	[Match loadMatchDataWithCompletionHandler : ^ (NSData* data, NSError* error)
	{
		if (DownloadCallback.IsBound())
		{
			DownloadCallback.Execute(GetMatchID(), error == nil);
		}
	}];
}

bool FTurnBasedMatchIOS::HasMatchData() const
{
	return Match.matchData != nil && Match.matchData.length != 0;
}

bool FTurnBasedMatchIOS::GetMatchData(TArray<uint8>& OutMatchData) const
{
	if (!HasMatchData())
	{
		return false;
	}

	int32 dataSize = Match.matchData.length;
	OutMatchData.Empty(dataSize);
	OutMatchData.AddUninitialized(dataSize - OutMatchData.Num());
	FMemory::Memcpy(OutMatchData.GetData(), Match.matchData.bytes, dataSize);
	return true;
}

void FTurnBasedMatchIOS::SetMatchData(const TArray<uint8>& NewMatchData, FUploadMatchDataSignature UploadCallback)
{
	[Match saveCurrentTurnWithMatchData : [NSData dataWithBytes : NewMatchData.GetData() length : NewMatchData.Num()] completionHandler : ^ (NSError* error)
	{
		UploadCallback.Execute(GetMatchID(), error == nil);
	}];
}

FString FTurnBasedMatchIOS::GetMatchID() const
{
	if (!Match)
	{
		return FTurnBasedMatch::GetMatchID();
	}
	return FString::Printf(TEXT("%s"), UTF8_TO_TCHAR(Match.matchID.UTF8String));
}

int32 FTurnBasedMatchIOS::GetLocalPlayerIndex() const
{
	IOnlineSubsystem* OSS = IOnlineSubsystem::Get();
	IOnlineIdentityPtr IdentityInterface = OSS ? OSS->GetIdentityInterface() : NULL;
	if (!IdentityInterface.IsValid())
	{
		UE_LOGF(LogTurnBasedInterfaceIOS, Warning, "No Online Identity");
		return 0;
	}

	FUniqueNetIdPtr NetID = IdentityInterface->GetUniquePlayerId(0);
	NSString* playerID = [NSString stringWithFormat : @"%s", TCHAR_TO_UTF8(*(NetID->ToString()))];

	int32 PlayerIndex = 0;
	NSArray* participantArray = Match.participants;
	for (GKTurnBasedParticipant* participant in participantArray)
	{
        NSString* PlayerIDString = nil;
        if ([GKTurnBasedParticipant respondsToSelector:@selector(player)] == YES)
        {
			PlayerIDString = FOnlineSubsystemIOS::GetPlayerId(participant.player);
        }
		if ([playerID isEqualToString : PlayerIDString])
		{
			return PlayerIndex;
		}
		++PlayerIndex;
	}
	return PlayerIndex;
}

int32 FTurnBasedMatchIOS::GetCurrentPlayerIndex() const
{
	GKTurnBasedParticipant* currentParticipant = Match.currentParticipant;
	if (!currentParticipant)
	{
		return 0;
	}

	return[Match.participants indexOfObject : currentParticipant];
}

EMPMatchOutcome::Outcome FTurnBasedMatchIOS::GetMatchOutcomeForPlayer(int32 PlayerIndex) const
{
	if (PlayerIndex >= Match.participants.count)
	{
		return EMPMatchOutcome::None;
	}

	GKTurnBasedParticipant* participant = Match.participants[PlayerIndex];
	GKTurnBasedMatchOutcome GKOutcome = participant.matchOutcome;
	return GetMatchOutcomeFromGKTurnBasedMatchOutcome(GKOutcome);
}

int32 FTurnBasedMatchIOS::GetPlayerIndexForPlayer(NSString* PlayerID) const
{
	int32 playerIndex = 0;
	for (GKTurnBasedParticipant* participant in Match.participants)
	{
        NSString* PlayerIDString = nil;
        if ([GKTurnBasedParticipant respondsToSelector:@selector(player)] == YES)
        {
			PlayerIDString = FOnlineSubsystemIOS::GetPlayerId(participant.player);
        }
		if ([PlayerIDString isEqualToString : PlayerID])
		{
			return playerIndex;
		}
	}
	UE_LOGF(LogTurnBasedInterfaceIOS, Warning, "Failed to find participant %ls in match", StringCast<TCHAR>(reinterpret_cast<const UTF8CHAR*>(PlayerID.UTF8String)).Get());
	return 0;
}

bool FTurnBasedMatchIOS::IsGKTurnBasedMatch(GKTurnBasedMatch* Comparison) const
{
	if (!Comparison || !Match)
	{
		return false;
	}

	return[Comparison.matchID isEqualToString : Match.matchID];
}

void FTurnBasedMatchIOS::EndTurnWithMatchData(const TArray<uint8>& MatchData, int32 TurnTimeoutInSeconds, FUploadMatchDataSignature UploadCallback)
{
	NSArray* participantArray = GetNextParticipantArray(INCLUDE_LOCAL_PLAYER);
	[Match endTurnWithNextParticipants : participantArray
	turnTimeout : TurnTimeoutInSeconds
			  matchData : [NSData dataWithBytes : MatchData.GetData() length : MatchData.Num()]
					  completionHandler : ^ (NSError* error)
	{
		if (UploadCallback.IsBound())
		{
			UploadCallback.Execute(GetMatchID(), error == nil);
		}
	}];
}

void FTurnBasedMatchIOS::QuitMatch(EMPMatchOutcome::Outcome Outcome, int32 TurnTimeoutInSeconds, FQuitMatchSignature QuitMatchCallback)
{
	int32 localPlayerIndex = GetLocalPlayerIndex();
	if (localPlayerIndex < Match.participants.count)
	{
		GKTurnBasedParticipant* localParticipant = Match.participants[localPlayerIndex];
		if (localParticipant.matchOutcome != GKTurnBasedMatchOutcomeNone)
		{
			return;
		}
	}

	GKTurnBasedMatchOutcome GKOutcome = GetGKTurnBasedMatchOutcomeFromMatchOutcome(Outcome);
	if (localPlayerIndex == GetCurrentPlayerIndex())
	{
		QuitMatchInTurn(GKOutcome, TurnTimeoutInSeconds, QuitMatchCallback);
	}
	else
	{
		QuitMatchOutOfTurn(GKOutcome, QuitMatchCallback);
	}
}

void FTurnBasedMatchIOS::QuitMatchInTurn(GKTurnBasedMatchOutcome Outcome, int32 TurnTimeoutInSeconds, FQuitMatchSignature QuitMatchCallback)
{
	FString MatchID = GetMatchID();
	NSArray* participantArray = GetNextParticipantArray(DO_NOT_INCLUDE_LOCAL_PLAYER);
	[Match participantQuitInTurnWithOutcome : Outcome nextParticipants : participantArray turnTimeout : TurnTimeoutInSeconds matchData : Match.matchData completionHandler : ^ (NSError* error)
	{
		if (QuitMatchCallback.IsBound())
		{
			QuitMatchCallback.Execute(MatchID, error == nil);
		}
	}];
}

void FTurnBasedMatchIOS::QuitMatchOutOfTurn(GKTurnBasedMatchOutcome Outcome, FQuitMatchSignature QuitMatchCallback)
{
	FString MatchID = GetMatchID();
	[Match participantQuitOutOfTurnWithOutcome : Outcome withCompletionHandler : ^ (NSError* error)
	{
		if (QuitMatchCallback.IsBound())
		{
			QuitMatchCallback.Execute(MatchID, error == nil);
		}
	}];
}

NSArray* FTurnBasedMatchIOS::GetNextParticipantArray(bool IncludeLocalPlayer) const
{
	NSArray* participantArray = Match.participants;

	int32 localPlayerIndex = GetLocalPlayerIndex();
	int32 nextPlayerIndex = (localPlayerIndex + 1) % participantArray.count;
	NSMutableArray* nextParticipantArray = [NSMutableArray array];
	while (nextPlayerIndex != localPlayerIndex)
	{
		[nextParticipantArray addObject : [participantArray objectAtIndex : nextPlayerIndex]];
		nextPlayerIndex = (nextPlayerIndex + 1) % participantArray.count;
	}

	if (IncludeLocalPlayer)
	{
		[nextParticipantArray addObject : [participantArray objectAtIndex : localPlayerIndex]];
	}

	return nextParticipantArray;
}

GKTurnBasedMatchOutcome FTurnBasedMatchIOS::GetGKTurnBasedMatchOutcomeFromMatchOutcome(EMPMatchOutcome::Outcome Outcome) const
{
	switch (Outcome)
	{
	default:
	case EMPMatchOutcome::None:        return GKTurnBasedMatchOutcomeNone;
	case EMPMatchOutcome::Quit:        return GKTurnBasedMatchOutcomeQuit;
	case EMPMatchOutcome::Won:         return GKTurnBasedMatchOutcomeWon;
	case EMPMatchOutcome::Lost:        return GKTurnBasedMatchOutcomeLost;
	case EMPMatchOutcome::Tied:        return GKTurnBasedMatchOutcomeTied;
	case EMPMatchOutcome::TimeExpired: return GKTurnBasedMatchOutcomeTimeExpired;
	case EMPMatchOutcome::First:       return GKTurnBasedMatchOutcomeFirst;
	case EMPMatchOutcome::Second:      return GKTurnBasedMatchOutcomeSecond;
	case EMPMatchOutcome::Third:       return GKTurnBasedMatchOutcomeThird;
	case EMPMatchOutcome::Fourth:      return GKTurnBasedMatchOutcomeFourth;
	}
}

EMPMatchOutcome::Outcome FTurnBasedMatchIOS::GetMatchOutcomeFromGKTurnBasedMatchOutcome(GKTurnBasedMatchOutcome GKOutcome) const
{
	switch (GKOutcome)
	{
	default:
	case GKTurnBasedMatchOutcomeNone:           return EMPMatchOutcome::None;
	case GKTurnBasedMatchOutcomeQuit:           return EMPMatchOutcome::Quit;
	case GKTurnBasedMatchOutcomeWon:            return EMPMatchOutcome::Won;
	case GKTurnBasedMatchOutcomeLost:           return EMPMatchOutcome::Lost;
	case GKTurnBasedMatchOutcomeTied:           return EMPMatchOutcome::Tied;
	case GKTurnBasedMatchOutcomeTimeExpired:    return EMPMatchOutcome::TimeExpired;
	case GKTurnBasedMatchOutcomeFirst:          return EMPMatchOutcome::First;
	case GKTurnBasedMatchOutcomeSecond:         return EMPMatchOutcome::Second;
	case GKTurnBasedMatchOutcomeThird:          return EMPMatchOutcome::Third;
	case GKTurnBasedMatchOutcomeFourth:         return EMPMatchOutcome::Fourth;
	}
}

void FTurnBasedMatchIOS::SetGKMatch(GKTurnBasedMatch* GKMatch)
{
	[Match release];
	Match = [GKMatch retain];
}

void FTurnBasedMatchIOS::EndMatch(FEndMatchSignature EndMatchCallback, EMPMatchOutcome::Outcome LocalPlayerOutcome, EMPMatchOutcome::Outcome OtherPlayersOutcome)
{
	FString MatchID = GetMatchID();

	for (int32 i = 0; i < Match.participants.count; ++i)
	{
		GKTurnBasedParticipant* participant = Match.participants[i];
		GKTurnBasedMatchOutcome GKOutcome = participant.matchOutcome;
		if (GKOutcome == GKTurnBasedMatchOutcomeNone)
		{
            if (GetLocalPlayerIndex() == i)
            {
				participant.matchOutcome = GetGKTurnBasedMatchOutcomeFromMatchOutcome(LocalPlayerOutcome);
            }
            else
            {
				participant.matchOutcome = GetGKTurnBasedMatchOutcomeFromMatchOutcome(OtherPlayersOutcome);
            }
			
		}
	}

	[Match endMatchInTurnWithMatchData : Match.matchData completionHandler : ^ (NSError* error)
	{
		if (EndMatchCallback.IsBound())
		{
			EndMatchCallback.Execute(MatchID, error == nil);
		}
	}];
}

FOnlineTurnBasedIOS::FOnlineTurnBasedIOS()
	: IOnlineTurnBased()
	, FTurnBasedMatchmakerDelegate()
	, Matchmaker(*this)
	, MatchmakerDelegate(nullptr)
	, EventListener(nil)
	, EventDelegate(nullptr)
	, NumberOfMatchesBeingLoaded(0)
{
	EventListener = [[FTurnBasedEventListenerIOS alloc] initWithOwner:*this];
}

FOnlineTurnBasedIOS::~FOnlineTurnBasedIOS()
{
	[EventListener release];
}

void FOnlineTurnBasedIOS::SetMatchmakerDelegate(FTurnBasedMatchmakerDelegatePtr Delegate)
{
	MatchmakerDelegate = Delegate;
}

void FOnlineTurnBasedIOS::ShowMatchmaker(const FTurnBasedMatchRequest& MatchRequest)
{
	Matchmaker.ShowWithMatchRequest(MatchRequest);
}

void FOnlineTurnBasedIOS::SetEventDelegate(FTurnBasedEventDelegateWeakPtr Delegate)
{
	EventDelegate = Delegate;
	if (!EventListener && Delegate.IsValid())
	{
		EventListener = [[FTurnBasedEventListenerIOS alloc] initWithOwner:*this];
	}
	else if (EventListener && !Delegate.IsValid())
	{
		[EventListener release];
		EventListener = nil;
	}
}

FTurnBasedEventDelegateWeakPtr FOnlineTurnBasedIOS::GetEventDelegate() const
{
	return EventDelegate;
}

void FOnlineTurnBasedIOS::LoadAllMatches(FLoadTurnBasedMatchesSignature MatchesLoadedCallback)
{
	if (NumberOfMatchesBeingLoaded > 0)
	{
		UE_LOGF(LogTurnBasedInterfaceIOS, Warning, "Requesting load all matches whilst we are still loading matches");
		return;
	}

	[GKTurnBasedMatch loadMatchesWithCompletionHandler : ^ (NSArray* matches, NSError* error)
	{
		MatchArray.Empty();

		NumberOfMatchesBeingLoaded = matches.count;
		for (GKTurnBasedMatch* match in matches)
		{
			NSArray* playerIdentifierArray = FOnlineTurnBasedIOS::GetPlayerIdentifierArrayForMatch(match);
			GKLocalPlayer* GKLocalUser = [GKLocalPlayer localPlayer];
			[GKLocalUser loadFriendsWithIdentifiers:(NSArray<NSString *> *)playerIdentifierArray completionHandler:^(NSArray<GKPlayer *> *players, NSError *nameLoadError)
			{
				if (!nameLoadError)
				{
					MatchArray.Add(MakeShareable(new FTurnBasedMatchIOS(match, players)));
				}

				--NumberOfMatchesBeingLoaded;
				if (NumberOfMatchesBeingLoaded == 0)
				{
					TArray<FString> MatchIDArray;
					for (TArray<FTurnBasedMatchRef>::TConstIterator It = MatchArray.CreateConstIterator(); It; ++It)
					{
						MatchIDArray.Add((*It)->GetMatchID());
					}

					MatchesLoadedCallback.ExecuteIfBound(MatchIDArray, error == nil);
				}
			}];
		}
	}];
}

void FOnlineTurnBasedIOS::LoadMatchWithID(FString MatchID, FLoadTurnBasedMatchWithIDSignature MatchLoadedCallback)
{
	NSString* IDString = [NSString stringWithUTF8String : TCHAR_TO_UTF8(*MatchID)];
	[GKTurnBasedMatch loadMatchWithID : IDString
	withCompletionHandler : ^ (GKTurnBasedMatch* match, NSError* error)
	{
		if (!error)
		{
			NSArray* playerIdentifierArray = FOnlineTurnBasedIOS::GetPlayerIdentifierArrayForMatch(match);
			GKLocalPlayer* GKLocalUser = [GKLocalPlayer localPlayer];
			[GKLocalUser loadFriendsWithIdentifiers:(NSArray<NSString *> *)playerIdentifierArray completionHandler:^(NSArray<GKPlayer *> *players, NSError *nameLoadError)
			{
				if (!nameLoadError)
				{
					FTurnBasedMatchPtr PreviousMatchPtr = GetMatchWithID(MatchID);
					if (PreviousMatchPtr.IsValid())
					{
						MatchArray.Remove(PreviousMatchPtr.ToSharedRef());
					}

					FTurnBasedMatchRef NewMatch = MakeShareable(new FTurnBasedMatchIOS(match, players));
					MatchArray.Add(NewMatch);
					MatchLoadedCallback.ExecuteIfBound(NewMatch->GetMatchID(), true);
				}
				else
				{
					MatchLoadedCallback.ExecuteIfBound(TEXT(""), false);
				}
			}];
		}
		else
		{
			MatchLoadedCallback.ExecuteIfBound(TEXT(""), false);
		}
	}];
}

FTurnBasedMatchPtr FOnlineTurnBasedIOS::GetMatchWithID(FString MatchID) const
{
	for (TArray<FTurnBasedMatchRef>::TConstIterator It = MatchArray.CreateConstIterator(); It; ++It)
	{
		FTurnBasedMatch& Match = (*It).Get();
		if (Match.GetMatchID().Compare(MatchID) == 0)
		{
			return *It;
		}
	}
	return nullptr;
}

void FOnlineTurnBasedIOS::RemoveMatch(FTurnBasedMatchRef Match, FRemoveMatchSignature RemoveMatchCallback)
{
	FTurnBasedMatchIOS& MatchIOS = static_cast<FTurnBasedMatchIOS&>(Match.Get());

	FString MatchID = MatchIOS.GetMatchID();
	[MatchIOS.GetGKMatch() removeWithCompletionHandler:^ (NSError* error)
	{
		MatchArray.Remove(Match);
		if (RemoveMatchCallback.IsBound())
		{
			RemoveMatchCallback.Execute(MatchID, error == nil);
		}
	}];
}

void FOnlineTurnBasedIOS::OnMatchmakerCancelled()
{
	if (MatchmakerDelegate.IsValid())
	{
		MatchmakerDelegate.Pin()->OnMatchmakerCancelled();
	}
}

void FOnlineTurnBasedIOS::OnMatchmakerFailed()
{
	if (!MatchmakerDelegate.IsValid())
	{
		MatchmakerDelegate.Pin()->OnMatchmakerFailed();
	}
}

void FOnlineTurnBasedIOS::OnMatchFound(FTurnBasedMatchRef Match)
{
	MatchArray.Add(Match);

	if (MatchmakerDelegate.IsValid())
	{
		dispatch_async(dispatch_get_main_queue(), ^
		{
			[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
			{
				MatchmakerDelegate.Pin()->OnMatchFound(Match);
				return true;
			}];

		});
	}
}

void FOnlineTurnBasedIOS::OnMatchEnded(FString MatchID)
{
	dispatch_async(dispatch_get_main_queue(), ^
	{
        [FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
        {
            if (TurnBasedMatchInterfaceObject)
            {
                   ITurnBasedMatchInterface::Execute_OnMatchEnded(TurnBasedMatchInterfaceObject, MatchID);
            }
            if (EventDelegate.IsValid())
            {
                EventDelegate.Pin()->OnMatchEnded(MatchID);
            }
            return true;
         }];
	});
}

void FOnlineTurnBasedIOS::OnMatchReceivedTurnEvent(FString MatchID, bool BecameActive, void* Match)
{
	GKTurnBasedMatch* IOSTurnBasedMatch = (GKTurnBasedMatch*)Match;
	
	NSArray* PlayerIdentifierArray = FOnlineTurnBasedIOS::GetPlayerIdentifierArrayForMatch(IOSTurnBasedMatch);
	GKLocalPlayer* GKLocalUser = [GKLocalPlayer localPlayer];
	[GKLocalUser loadFriendsWithIdentifiers:(NSArray<NSString *> *)PlayerIdentifierArray completionHandler:^(NSArray<GKPlayer *> *Players, NSError *NameLoadError)
	{
		if (!NameLoadError)
		{
			FTurnBasedMatchPtr PreviousMatchPtr = GetMatchWithID(MatchID);
			if (PreviousMatchPtr.IsValid())
			{
				MatchArray.Remove(PreviousMatchPtr.ToSharedRef());
			}
			
			FTurnBasedMatchRef NewMatch = MakeShareable(new FTurnBasedMatchIOS(IOSTurnBasedMatch, Players));
			MatchArray.Add(NewMatch);
			dispatch_async(dispatch_get_main_queue(), ^
						   {
				[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
				 {
					if (TurnBasedMatchInterfaceObject)
					{
						TArray<uint8> MatchData;
						if (GetMatchWithID(MatchID)->GetMatchData(MatchData))
						{
							// TODO: We should cache off the RepLayout by class, just like we do in NetDriver.
							FBitReader Reader(MatchData.GetData(), MATCH_DATA_SIZE);
							FRepLayout::CreateFromClass(TurnBasedMatchInterfaceObject->GetClass())->SerializeObjectReplicatedProperties(TurnBasedMatchInterfaceObject, Reader);
						}
						ITurnBasedMatchInterface::Execute_OnMatchReceivedTurn(TurnBasedMatchInterfaceObject, MatchID, BecameActive);
					}
					if (EventDelegate.IsValid())
					{
						EventDelegate.Pin()->OnMatchReceivedTurnEvent(MatchID, BecameActive, Match);
					}
					return true;
				}];
				
			});
		}
	}];
}

NSArray* FOnlineTurnBasedIOS::GetPlayerIdentifierArrayForMatch(GKTurnBasedMatch* match)
{
	NSMutableArray* result = [NSMutableArray array];
	for (GKTurnBasedParticipant* participant in match.participants)
	{
        NSString* PlayerIDString = nil;
        if ([GKTurnBasedParticipant respondsToSelector:@selector(player)] == YES)
        {
			PlayerIDString = FOnlineSubsystemIOS::GetPlayerId(participant.player);
        }
		if (!PlayerIDString)
		{
			break;
		}
		[result addObject : PlayerIDString];
	}
	return result;
}

void FOnlineTurnBasedIOS::RegisterTurnBasedMatchInterfaceObject(UObject* Object)
{
	if (Object != nullptr && Object->GetClass()->ImplementsInterface(UTurnBasedMatchInterface::StaticClass()))
	{
		TurnBasedMatchInterfaceObject = Object;
	}
}
