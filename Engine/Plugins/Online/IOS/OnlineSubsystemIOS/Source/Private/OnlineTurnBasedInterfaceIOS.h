// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineTurnBasedInterface.h"
#include "TurnBasedMatchmakerIOS.h"
#import <GameKit/GameKit.h>

@class FTurnBasedEventListenerIOS;

// Wrapper for a Game Center Match
class FTurnBasedMatchIOS : public FTurnBasedMatch
{
public:
	FTurnBasedMatchIOS(GKTurnBasedMatch* _Match, NSArray* PlayerArray);
	virtual ~FTurnBasedMatchIOS();

	// Returns the number of players currently participating in a match
	virtual int32 GetNumberOfPlayers() const override;

	// Takes the index of a player PlayerIndex, and sets the name of that player in Name.
	// Return true if the PlayerIndex is valid, false if not.
	virtual bool GetPlayerDisplayName(int32 PlayerIndex, FString& Name) const override;

	// Reloads the custom data for this match.
	virtual void ReloadMatchData(FDownloadMatchDataSignature DownloadCallback) override;

	// Accessors for the custom data of this match. Custom match data is application specific and can be anything,
	// up to 8k.
	virtual bool HasMatchData() const override;
	virtual bool GetMatchData(TArray<uint8>& OutMatchData) const override;
	virtual void SetMatchData(const TArray<uint8>& NewMatchData, FUploadMatchDataSignature UploadCallback) override;

	// Returns the GUID of this match as an FString.
	virtual FString GetMatchID() const override;

	// Get the index of the local player in the participants array of this match.
	virtual int32 GetLocalPlayerIndex() const override;

	// Get the index of the active player in the participants array of this match.
	virtual int32 GetCurrentPlayerIndex() const override;

	// Gets the match outcome for a player (win/loss/quit/etc.)
	virtual EMPMatchOutcome::Outcome GetMatchOutcomeForPlayer(int32 PlayerIndex) const override;

	// Gets the player index in the participants array of the player with PlayerID.
	int32 GetPlayerIndexForPlayer(NSString* PlayerID) const;

	// Returns true if the match stored in this FTurnBasedMatchIOS object is the same as Comparison.
	bool IsGKTurnBasedMatch(GKTurnBasedMatch* Comparison) const;

	// Ends the turn for the current player with updated MatchData. TurnTimeoutInSeconds is how long the next player has for their turn, and
	// UploadCallback will be called when the GameCenter call finishes.
	virtual void EndTurnWithMatchData(const TArray<uint8>& MatchData, int32 TurnTimeoutInSeconds, FUploadMatchDataSignature UploadCallback) override;

	// Quits this match with the passed in Outcome. TurnTimeoutInSeconds is how long the next player will have (if the player is quitting early
	// and the match isn't over).
	virtual void QuitMatch(EMPMatchOutcome::Outcome Outcome, int32 TurnTimeoutInSeconds, FQuitMatchSignature QuitMatchCallback) override;

	// Accessors for the GKTurnBasedMatch object.
	GKTurnBasedMatch* GetGKMatch() const { return Match; }
	void SetGKMatch(GKTurnBasedMatch* GKMatch);

	// Ends the match
	virtual void EndMatch(FEndMatchSignature QuitMatchCallback, EMPMatchOutcome::Outcome LocalPlayerOutcome, EMPMatchOutcome::Outcome OtherPlayersOutcome);

private:

	// Helper functions for QuitMatch, depending on whether it is the current player's turn or not.
	void QuitMatchInTurn(GKTurnBasedMatchOutcome Outcome, int32 TurnTimeoutInSeconds, FQuitMatchSignature QuitMatchCallback);
	void QuitMatchOutOfTurn(GKTurnBasedMatchOutcome Outcome, FQuitMatchSignature QuitMatchCallback);

	// Gets a list of players, if IncludeLocalPlayer is true, this player will be in the array
	NSArray* GetNextParticipantArray(bool IncludeLocalPlayer) const;

	// Conversion functions between EMPMatchOutcome::Outcome and GKTurnBasedMatchOutcome enums.
	GKTurnBasedMatchOutcome GetGKTurnBasedMatchOutcomeFromMatchOutcome(EMPMatchOutcome::Outcome Outcome) const;
	EMPMatchOutcome::Outcome GetMatchOutcomeFromGKTurnBasedMatchOutcome(GKTurnBasedMatchOutcome GKOutcome) const;

	// The last of active player GUIDs in the match
	TArray<FString> PlayerAliasArray;

	// The Game Center match object
	GKTurnBasedMatch* Match;
};

/**
*	FOnlineTurnBasedIOS - Implementation of turn based multiplayer for IOS
*/
class FOnlineTurnBasedIOS : public IOnlineTurnBased, public FTurnBasedMatchmakerDelegate, public FTurnBasedEventDelegate
{
public:
	FOnlineTurnBasedIOS();
	virtual ~FOnlineTurnBasedIOS();

	static NSArray* GetPlayerIdentifierArrayForMatch(GKTurnBasedMatch* match);

	// IOnlineTurnBased
	virtual void SetMatchmakerDelegate(FTurnBasedMatchmakerDelegatePtr Delegate) override;
	virtual void ShowMatchmaker(const FTurnBasedMatchRequest& MatchRequest) override;

	virtual void SetEventDelegate(FTurnBasedEventDelegateWeakPtr Delegate) override;
	virtual FTurnBasedEventDelegateWeakPtr GetEventDelegate() const override;

	virtual void LoadAllMatches(FLoadTurnBasedMatchesSignature MatchesLoadedCallback) override;
	virtual void LoadMatchWithID(FString MatchID, FLoadTurnBasedMatchWithIDSignature MatchesLoadedCallback) override;

	virtual FTurnBasedMatchPtr GetMatchWithID(FString MatchID) const override;

	virtual void RemoveMatch(FTurnBasedMatchRef Match, FRemoveMatchSignature RemoveMatchCallback) override;

	// FTurnBasedMatchmakerDelegate
	virtual void OnMatchmakerCancelled() override;
	virtual void OnMatchmakerFailed() override;
	virtual void OnMatchFound(FTurnBasedMatchRef Match) override;

	// FTurnBasedEventDelegate
	virtual void OnMatchEnded(FString MatchID) override;
	virtual void OnMatchReceivedTurnEvent(FString MatchID, bool BecameActive, void* Match) override;

	virtual void RegisterTurnBasedMatchInterfaceObject(UObject* Object);
	UObject* GetTurnBasedMatchInterfaceObject() { return TurnBasedMatchInterfaceObject; }

	virtual int32 GetMatchDataSize() { return MATCH_DATA_SIZE; }

private:
	FTurnBasedMatchmakerIOS Matchmaker;
	FTurnBasedMatchmakerDelegateWeakPtr MatchmakerDelegate;

	FTurnBasedEventListenerIOS* EventListener;
	FTurnBasedEventDelegateWeakPtr EventDelegate;

	TArray<FTurnBasedMatchRef> MatchArray;

	int32 NumberOfMatchesBeingLoaded;

	UObject* TurnBasedMatchInterfaceObject;

	static const int32 MATCH_DATA_SIZE = 1024 * 8 * 4;
};

typedef TSharedPtr<FOnlineTurnBasedIOS, ESPMode::ThreadSafe> FOnlineTurnBasedIOSPtr;
