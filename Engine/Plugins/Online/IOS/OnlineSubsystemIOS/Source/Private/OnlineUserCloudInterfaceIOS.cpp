// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineUserCloudInterfaceIOS.h"
#include "Engine/Engine.h"
#include "HAL/PlatformProcess.h"
#include "Misc/CoreDelegates.h"
#include "Misc/FileHelper.h"
#include "Misc/ScopeLock.h"
#include "PlatformFeatures.h"
#include "GameFramework/GameUserSettings.h"

@implementation IOSCloudStorage

@synthesize CloudContainer;
@synthesize SharedDatabase;
@synthesize UserDatabase;
@synthesize iCloudToken;

-(IOSCloudStorage*)init:(bool)registerHandler
{
	if ([CKContainer class])
	{
		// get the current iCloud ubiquity token
		iCloudToken = [NSFileManager defaultManager].ubiquityIdentityToken;

		// register for iCloud change notifications
		if (registerHandler)
		{
			[[NSNotificationCenter defaultCenter] addObserver:self selector : @selector(iCloudAccountAvailabilityChanged:) name: NSUbiquityIdentityDidChangeNotification object : nil];
		}

		NSString *ICloudContainerIdentifier = [[NSBundle mainBundle].infoDictionary objectForKey : @"ICloudContainerIdentifier"];
		if (ICloudContainerIdentifier != nil)
		{
			NSLog(@"Using a custom CloudKit container: %@", ICloudContainerIdentifier);
			CloudContainer = [CKContainer containerWithIdentifier:ICloudContainerIdentifier];
		}
		else
		{
			CloudContainer = [CKContainer defaultContainer];
		}
		SharedDatabase = [CloudContainer publicCloudDatabase];
		UserDatabase = [CloudContainer privateCloudDatabase];
	}
	return self;
}

-(void)dealloc
{
	[CloudContainer release];
	[SharedDatabase release];
	[UserDatabase release];
	[iCloudToken release];
	[super dealloc];
}

-(bool)readFile:(NSString*)fileName sharedDB:(bool)shared completionHandler:(void(^)(CKRecord* record, NSError* error))handler
{
	if ([CKDatabase class])
	{
		CKDatabase* DB = shared ? SharedDatabase : UserDatabase;
		if (DB != nil)
		{
			CKRecordID* recordId = [[CKRecordID alloc] initWithRecordName:fileName];
			[DB fetchRecordWithID : recordId completionHandler : handler];
			return true;
		}
	}
	return false;
}

-(bool)writeFile:(NSString*)fileName contents:(NSData*)fileContents sharedDB:(bool)shared completionHandler:(void(^)(CKRecord* record, NSError* error))handler
{
	if ([CKDatabase class])
	{
		CKDatabase* DB = shared ? SharedDatabase : UserDatabase;
		if (DB != nil)
		{
			CKRecordID* recordId = [[[CKRecordID alloc] initWithRecordName:fileName] autorelease];
			CKRecord* record = [[CKRecord alloc] initWithRecordType:@"file" recordID: recordId];
			record[@"contents"] = fileContents;

			// use CKModifyRecordsOperation to allow updating existing records 
			CKModifyRecordsOperation *modifyRecords = [[CKModifyRecordsOperation alloc] initWithRecordsToSave:@[record] recordIDsToDelete:nil];
			modifyRecords.savePolicy = CKRecordSaveAllKeys;
			modifyRecords.qualityOfService = NSQualityOfServiceUserInitiated;
			
//			modifyRecords.perRecordCompletionBlock = handler; /* Deprecated iOS15. */
			// Since only 1 record is being saved per run, use the global completion block instead */
			modifyRecords.modifyRecordsCompletionBlock = ^(NSArray<CKRecord *> *savedRecords, NSArray<CKRecordID *> *deletedRecordIDs, NSError *operationError) {
				handler([savedRecords firstObject], operationError);
			};
			[DB addOperation : modifyRecords];

			return true;
		}
	}
	return false;
}

-(bool)deleteFile:(NSString*)fileName sharedDB:(bool)shared completionHandler:(void(^)(CKRecordID* record, NSError* error))handler
{
	if ([CKDatabase class])
	{
		CKDatabase* DB = shared ? SharedDatabase : UserDatabase;
		if (DB != nil)
		{
			CKRecordID* recordId = [[CKRecordID alloc] initWithRecordName:fileName];
			[DB deleteRecordWithID : recordId completionHandler : handler];
			return true;
		}
	}
	return false;
}

-(bool)query:(bool)shared fetchHandler:(void(^)(CKRecord* record))fetch completionHandler:(void(^)(CKQueryCursor* record, NSError* error))complete
{
	if ([CKDatabase class])
	{
		CKDatabase* DB = shared ? SharedDatabase : UserDatabase;
		if (DB != nil)
		{
			CKQuery* query = [[[CKQuery alloc] initWithRecordType:@"file" predicate:[NSPredicate predicateWithFormat : @"TRUEPREDICATE"]] autorelease];
			CKQueryOperation* queryOp = [[CKQueryOperation alloc] initWithQuery:query];
			queryOp.desiredKeys = @[@"record.recordID.recordName"];
//			queryOp.recordFetchedBlock = fetch;  /* Deprecated in iOS15.  Now performs once all records are loaded */
			queryOp.queryCompletionBlock = complete;
			queryOp.resultsLimit = CKQueryOperationMaximumResults;
			[DB addOperation : queryOp];
			return true;
		}
	}
	return false;
}

-(void)iCloudAccountAvailabilityChanged:(NSNotification*)notification
{
	// access the token
	id newiCloudToken = [NSFileManager defaultManager].ubiquityIdentityToken;

	// check to see what it has changed to
	if (newiCloudToken != iCloudToken)
	{
		[self init: false];
	}
}

+(IOSCloudStorage*)cloudStorage
{
	static IOSCloudStorage* theStorage = nil;

	if (theStorage == nil)
	{
		theStorage = [[IOSCloudStorage alloc] init:true];
	}
	return theStorage;
}

@end

//
// UserCloud interface implementation
//

FOnlineUserCloudInterfaceIOS::~FOnlineUserCloudInterfaceIOS()
{
	ClearFiles();
}

FCloudFile* FOnlineUserCloudInterfaceIOS::GetCloudFile(const FString& FileName, bool bCreateIfMissing)
{
    FScopeLock ScopeLock(&CloudDataLock);
	if (FileName.Len() > 0)
	{
		for (int32 FileIdx = 0; FileIdx < CloudFileData.Num(); FileIdx++)
		{
			FCloudFile* UserFileData = &CloudFileData[FileIdx];
			if (UserFileData &&
				UserFileData->FileName == FileName)
			{
				return UserFileData;
			}
		}

		if (bCreateIfMissing)
		{
			return new (CloudFileData)FCloudFile(FileName);
		}
	}

	return NULL;
}

FCloudFileHeader* FOnlineUserCloudInterfaceIOS::GetCloudFileHeader(const FString& FileName, bool bCreateIfMissing)
{
    FScopeLock ScopeLock(&CloudDataLock);
    if (FileName.Len() > 0)
    {
        for (int32 FileIdx = 0; FileIdx < CloudMetaData.Num(); FileIdx++)
        {
            FCloudFileHeader* UserFileData = &CloudMetaData[FileIdx];
            if (UserFileData &&
                UserFileData->FileName == FileName)
            {
                return UserFileData;
            }
        }
        
        if (bCreateIfMissing)
        {
            new (CloudMetaData)FCloudFileHeader(FileName, FileName, 0);
        }
    }
    
    return NULL;
}

bool FOnlineUserCloudInterfaceIOS::ClearFiles()
{
    FScopeLock ScopeLock(&CloudDataLock);
	// Delete file contents
	for (int32 FileIdx = 0; FileIdx < CloudFileData.Num(); FileIdx++)
	{
		CloudFileData[FileIdx].Data.Empty();
	}

	// No async files being handled, so empty them all
	CloudFileData.Empty();
	return true;
}

bool FOnlineUserCloudInterfaceIOS::ClearCloudFile(const FString& FileName)
{
    FScopeLock ScopeLock(&CloudDataLock);
	int32 FoundIndex = INDEX_NONE;
	for (int32 FileIdx = 0; FileIdx < CloudFileData.Num(); FileIdx++)
	{
		FCloudFile* UserFileData = &CloudFileData[FileIdx];
		if (UserFileData->FileName == FileName)
		{
			// If there is an async task outstanding, fail to empty
			if (UserFileData->AsyncState == EOnlineAsyncTaskState::InProgress)
			{
				return false;
			}

			UserFileData->Data.Empty();
			FoundIndex = FileIdx;
			break;
		}
	}

	if (FoundIndex != INDEX_NONE)
	{
		CloudFileData.RemoveAtSwap(FoundIndex);
	}
	return true;
}

bool FOnlineUserCloudInterfaceIOS::GetFileContents(const FUniqueNetId& UserId, const FString& FileName, TArray<uint8>& FileContents)
{
	FCloudFile* CloudFile = GetCloudFile(FileName);
	if (CloudFile && CloudFile->AsyncState == EOnlineAsyncTaskState::Done && CloudFile->Data.Num() > 0)
	{
		FileContents = CloudFile->Data;
		return true;
	}
	return false;
}

bool FOnlineUserCloudInterfaceIOS::ClearFiles(const FUniqueNetId& UserId)
{
	// NOTE: Return true regardless of if the user was valid or not. We don't care if
	// there weren't any files in need of clearing, only if there was a failure to clear
	ClearFiles();
	return true;
}

bool FOnlineUserCloudInterfaceIOS::ClearFile(const FUniqueNetId& UserId, const FString& FileName)
{
	// NOTE: Return true regardless of if the file exists or not. The only way to return false
	// would be in we failed to delete, and not deleting a non-existing file is not a failure
	ClearCloudFile(FileName);
	return true;
}

void FOnlineUserCloudInterfaceIOS::EnumerateUserFiles(const FUniqueNetId& UserId)
{
	MetaDataState = EOnlineAsyncTaskState::InProgress;
	if ([[IOSCloudStorage cloudStorage] query:false fetchHandler:^(CKRecord* record)
	{
         FString FileName = record.recordID.recordName;
         FCloudFileHeader* Header = GetCloudFileHeader(FileName, true);
	}
	completionHandler:^(CKQueryCursor* cursor, NSError* error)
	{
		if (error)
		{
			// TODO: record is potentially not found
            MetaDataState = EOnlineAsyncTaskState::Failed;
			TriggerOnEnumerateUserFilesCompleteDelegates(false, UserId);
			NSLog(@"Error: %@", error);
		}
		else
		{
			// store the contents in the memory record database
            MetaDataState = EOnlineAsyncTaskState::Done;
			TriggerOnEnumerateUserFilesCompleteDelegates(true, UserId);
			NSLog(@"Enumerated Read!");
		}
	}] == false)
    {
        TriggerOnEnumerateUserFilesCompleteDelegates(false, UserId);
    }
}

void FOnlineUserCloudInterfaceIOS::GetUserFileList(const FUniqueNetId& UserId, TArray<FCloudFileHeader>& UserFiles)
{
    while (MetaDataState == EOnlineAsyncTaskState::InProgress)
    {
        FPlatformProcess::Sleep(0.01f);
    }
    UserFiles = CloudMetaData;
}

bool FOnlineUserCloudInterfaceIOS::ReadUserFile(const FUniqueNetId& UserId, const FString& FileName)
{
	FCloudFile* CloudFile = GetCloudFile(FileName, true);
	if (CloudFile)
	{
        __block FString NewFile = FileName;
        CloudFile->AsyncState = EOnlineAsyncTaskState::InProgress;
		return [[IOSCloudStorage cloudStorage] readFile:FileName.GetNSString() sharedDB : false completionHandler : ^ (CKRecord *record, NSError *error)
		{
            FScopeLock ScopeLock(&CloudDataLock);
            FCloudFile* File = GetCloudFile(NewFile);
			if (error)
			{
				// TODO: record is potentially not found
				if (File)
				{
					File->AsyncState = EOnlineAsyncTaskState::Failed;
				}
				TriggerOnReadUserFileCompleteDelegates(false, UserId, NewFile);
				NSLog(@"Error: %@", error);
			}
			else
			{
 				// store the contents in the memory record database
				NSData* data = (NSData*)record[@"contents"];
				if (File && data)
				{
					File->Data.Empty();
					File->Data.Append((uint8*)data.bytes, data.length);
					File->AsyncState = EOnlineAsyncTaskState::Done;
				}
				TriggerOnReadUserFileCompleteDelegates(true, UserId, NewFile);
				NSLog(@"Record Read!");
			}
		}];
	}
    else
    {
        TriggerOnReadUserFileCompleteDelegates(false, UserId, FileName);
    }
	return false;
}

bool FOnlineUserCloudInterfaceIOS::WriteUserFile(const FUniqueNetId& UserId, const FString& FileName, TArray<uint8>& FileContents, bool bCompressBeforeUpload)
{
	FCloudFile* CloudFile = GetCloudFile(FileName, true);
	if (CloudFile)
	{
        __block TArray<uint8> DataContents = FileContents;
        __block FString NewFile = FileName;
		CloudFile->AsyncState = EOnlineAsyncTaskState::InProgress;
		return [[IOSCloudStorage cloudStorage] writeFile:FileName.GetNSString() contents : [[NSData alloc] initWithBytes:FileContents.GetData() length : FileContents.Num()] sharedDB : false completionHandler : ^ (CKRecord *record, NSError *error)
		{
            FScopeLock ScopeLock(&CloudDataLock);
            FCloudFile* File = GetCloudFile(NewFile);
			if (error)
			{
				// TODO: record is potentially newer on the server
				if (File)
				{
					File->AsyncState = EOnlineAsyncTaskState::Failed;
				}
				TriggerOnWriteUserFileCompleteDelegates(false, UserId, NewFile);
				NSLog(@"Error: %@", error);
			}
			else
			{
                FCloudFileHeader* Header = GetCloudFileHeader(NewFile, true);
				if (File)
				{
					File->Data = DataContents;
					File->AsyncState = EOnlineAsyncTaskState::Done;
				}
				TriggerOnWriteUserFileCompleteDelegates(true, UserId, NewFile);
				NSLog(@"Record Saved!");
			}
		}];
	}
    else
    {
        TriggerOnWriteUserFileCompleteDelegates(false, UserId, FileName);
    }
	return false;
}

void FOnlineUserCloudInterfaceIOS::CancelWriteUserFile(const FUniqueNetId& UserId, const FString& FileName)
{
	// Not implemented
}


bool FOnlineUserCloudInterfaceIOS::DeleteUserFile(const FUniqueNetId& UserId, const FString& FileName, bool bShouldCloudDelete, bool bShouldLocallyDelete)
{
	FCloudFile* CloudFile = GetCloudFile(FileName, true);
	if (CloudFile)
	{
        if (bShouldCloudDelete)
        {
            __block FString NewFile = FileName;
            __block bool bDeleteLocal = bShouldLocallyDelete;
            return [[IOSCloudStorage cloudStorage] deleteFile:FileName.GetNSString() sharedDB : false completionHandler : ^ (CKRecordID *record, NSError *error)
            {
                if (error)
                {
                    // TODO: record is potentially not found
                    TriggerOnDeleteUserFileCompleteDelegates(false, UserId, NewFile);
                    NSLog(@"Error: %@", error);
                }
                else
                {
                    // store the contents in the memory record database
                    if (bDeleteLocal)
                    {
                        ClearCloudFile(NewFile);
                    }
                    TriggerOnDeleteUserFileCompleteDelegates(true, UserId, NewFile);
                    NSLog(@"Record Deleted!");
                }
            }];
        }
        else if (bShouldLocallyDelete)
        {
            ClearCloudFile(FileName);
            TriggerOnDeleteUserFileCompleteDelegates(true, UserId, FileName);
            return true;
        }
	}
    else
    {
        TriggerOnDeleteUserFileCompleteDelegates(false, UserId, FileName);
    }
	return false;
}

bool FOnlineUserCloudInterfaceIOS::RequestUsageInfo(const FUniqueNetId& UserId)
{
	// Not implemented
	return false;
}

void FOnlineUserCloudInterfaceIOS::DumpCloudState(const FUniqueNetId& UserId)
{
	// Not implemented
}

void FOnlineUserCloudInterfaceIOS::DumpCloudFileState(const FUniqueNetId& UserId, const FString& FileName)
{
	// Not implemented
}

// cloud save implementation
void FOnlineUserCloudInterfaceIOS::InitCloudSave(bool InIOSAlwaysSyncCloudFiles)
{
	SaveSystem = (FIOSSaveGameSystem*)IPlatformFeaturesModule::Get().GetSaveGameSystem();

	check(SaveSystem);

	bIOSAlwaysSyncCloudFiles = InIOSAlwaysSyncCloudFiles;

	OnEnumerateUserCloudFilesCompleteDelegate = FOnEnumerateUserFilesCompleteDelegate::CreateRaw(this, &FOnlineUserCloudInterfaceIOS::OnEnumerateUserFilesComplete);
	OnInitialFetchUserCloudFileCompleteDelegate = FOnReadUserFileCompleteDelegate::CreateRaw(this, &FOnlineUserCloudInterfaceIOS::OnInitialFetchUserCloudFileComplete);
	OnWriteUserCloudFileCompleteDelegate = FOnWriteUserFileCompleteDelegate::CreateRaw(this, &FOnlineUserCloudInterfaceIOS::OnWriteUserCloudFileComplete);
	OnReadUserCloudFileCompleteDelegate = FOnReadUserFileCompleteDelegate::CreateRaw(this, &FOnlineUserCloudInterfaceIOS::OnReadUserCloudFileComplete);
	OnDeleteUserCloudFileCompleteDelegate = FOnDeleteUserFileCompleteDelegate::CreateRaw(this, &FOnlineUserCloudInterfaceIOS::OnDeleteUserCloudFileComplete);

	// link delegates to the iOS save system
	SaveSystem->OnUpdateCloudDataFromLocalSave = FIOSSaveGameSystem::FUpdateCloudDataFromLocalSave::CreateRaw(this, &FOnlineUserCloudInterfaceIOS::OnWriteUserCloudFileBegin);
	SaveSystem->OnUpdateLocalSaveFileFromCloud = FIOSSaveGameSystem::FUpdateLocalSaveFileFromCloud::CreateRaw(this, &FOnlineUserCloudInterfaceIOS::OnReadUserCloudFileBegin);
	SaveSystem->OnDeleteCloudData = FIOSSaveGameSystem::FDeleteCloudData::CreateRaw(this, &FOnlineUserCloudInterfaceIOS::OnDeleteUserCloudFileBegin);

	UniqueNetId = FUniqueNetIdIOS::Create();

	UpdateDictionary = [NSMutableDictionary new];
	
	OnEnumerateUserCloudFilesCompleteDelegateHandle = AddOnEnumerateUserFilesCompleteDelegate_Handle(OnEnumerateUserCloudFilesCompleteDelegate);
	EnumerateUserFiles(*UniqueNetId);
    
    auto RegisterItems = [](auto classToCall)
    {
		UGameUserSettings* UserSettings;
        UserSettings = GEngine->GetGameUserSettings();
        check(UserSettings);

    	UserSettings->OnUpdateCloudDataFromGameUserSettings = UGameUserSettings::FUpdateCloudDataFromGameUserSettings::CreateRaw(classToCall, &FOnlineUserCloudInterfaceIOS::OnWriteUserCloudFileBegin);
        UserSettings->OnUpdateGameUserSettingsFileFromCloud = UGameUserSettings::FUpdateGameUserSettingsFileFromCloud::CreateRaw(classToCall, &FOnlineUserCloudInterfaceIOS::OnReadUserCloudFileBegin);
    };

    if (GEngine)
    {
        RegisterItems(this);
    }
    else
    {
		FCoreDelegates::GetOnPostEngineInit().AddLambda(RegisterItems, this);
    }
}

void FOnlineUserCloudInterfaceIOS::OnEnumerateUserFilesComplete(bool bWasSuccessful, const FUniqueNetId & UserId)
{
	ClearOnEnumerateUserFilesCompleteDelegate_Handle(OnEnumerateUserCloudFilesCompleteDelegateHandle);

	TArray<FCloudFileHeader> UserFiles;
	if (bWasSuccessful)
	{
		GetUserFileList(UserId, UserFiles);

		for (int32 Idx = 0; Idx < UserFiles.Num(); Idx++)
		{
			OnReadUserCloudFileCompleteDelegateHandle = AddOnReadUserFileCompleteDelegate_Handle(OnInitialFetchUserCloudFileCompleteDelegate);
			ReadUserFile(UserId, UserFiles[Idx].FileName);
		}
	}
}

void FOnlineUserCloudInterfaceIOS::OnWriteUserCloudFileComplete(bool bWasSuccessful, const FUniqueNetId & UserId, const FString & FileName)
{
	ClearOnWriteUserFileCompleteDelegate_Handle(OnWriteUserCloudFileCompleteDelegateHandle);

	if (bWasSuccessful)
	{
		//flag that we have the latest record
		[UpdateDictionary setObject : [NSNumber numberWithBool : false]  forKey : FileName.GetNSString()];
	}
	
	// clean up temporary data
	ClearFile(UserId, FileName);
}

void FOnlineUserCloudInterfaceIOS::OnInitialFetchUserCloudFileComplete(bool bWasSuccessful, const FUniqueNetId & UserId, const FString & FileName)
{
	OnReadUserCloudFileComplete(bWasSuccessful, UserId, FileName);

	if (bWasSuccessful)
	{
		//flag that we have the latest record
		[UpdateDictionary setObject : [NSNumber numberWithBool : false]  forKey : FileName.GetNSString()];
	}
		
	// clean up temporary data
	ClearFile(UserId, FileName);
}

void FOnlineUserCloudInterfaceIOS::OnReadUserCloudFileComplete(bool bWasSuccessful, const FUniqueNetId & UserId, const FString & FileName)
{
	ClearOnReadUserFileCompleteDelegate_Handle(OnReadUserCloudFileCompleteDelegateHandle);

	if (bWasSuccessful)
	{
		// locally cache the server data
		TArray<uint8> FileContents;
		GetFileContents(UserId, FileName, FileContents);
        FFileHelper::SaveArrayToFile(FileContents, *FileName);
		
		//flag that we have the latest record
		[UpdateDictionary setObject : [NSNumber numberWithBool : false]  forKey : FileName.GetNSString()];
	}
}

void FOnlineUserCloudInterfaceIOS::OnDeleteUserCloudFileComplete(bool bWasSuccessful, const FUniqueNetId & UserId, const FString & FileName)
{
	ClearOnDeleteUserFileCompleteDelegate_Handle(OnDeleteUserCloudFileCompleteDelegateHandle);
}

bool FOnlineUserCloudInterfaceIOS::OnWriteUserCloudFileBegin(const FString &  FileName)
{
    OnWriteUserCloudFileCompleteDelegateHandle = AddOnWriteUserFileCompleteDelegate_Handle(OnWriteUserCloudFileCompleteDelegate);
    
    TArray<uint8> FileData;
    
    if (FFileHelper::LoadFileToArray(FileData, *FileName))
    {
        return WriteUserFile(*UniqueNetId, FileName, FileData);
    }
    
    return false;
}

bool FOnlineUserCloudInterfaceIOS::ShouldFetchRecordFromCloud(const FString &  FileName)
{
#if !PLATFORM_TVOS
	if (!bIOSAlwaysSyncCloudFiles)
	{
		NSNumber *value = [UpdateDictionary objectForKey : FileName.GetNSString()];
		if (value)
		{
			return[value boolValue];
		}
	}
#endif
	return true;
}

bool FOnlineUserCloudInterfaceIOS::OnReadUserCloudFileBegin(const FString & FileName)
{
#if !PLATFORM_TVOS
	if (ShouldFetchRecordFromCloud(FileName))
#endif
	{
		OnReadUserCloudFileCompleteDelegateHandle = AddOnReadUserFileCompleteDelegate_Handle(OnReadUserCloudFileCompleteDelegate);

		// check cloudkit was properly initialized
		while (MetaDataState == EOnlineAsyncTaskState::InProgress)
		{
			FPlatformProcess::Sleep(0.01f);	// Cloudkit still needs to finish initialization
		}
		if (MetaDataState == EOnlineAsyncTaskState::Failed)
		{
			return false; // Cloudkit has failed to initialize for some reason
		}
		ensureMsgf(MetaDataState == EOnlineAsyncTaskState::Done, TEXT("CloudKit should be initialized first"));

		// check named record in Cloudkit
		if (!ReadUserFile(*UniqueNetId, FileName))
		{
			return false; // ReadUserFile failed
		}
		bool bLoop = true;
		while (bLoop)
		{
			// loop until the async task set in ReadUserFile is finished (Done or Failed)
			{
				FScopeLock ScopeLock(&CloudDataLock);
				FCloudFile* CloudFile = GetCloudFile(FileName, false);
				bLoop = (CloudFile) ? (CloudFile->AsyncState == EOnlineAsyncTaskState::InProgress) : false;
			}
			if (bLoop)
			{
				FPlatformProcess::Sleep(0.01f);
			}
		}

		// clean up temporary data
		ClearFile(*UniqueNetId, FileName);
	}
    return true;
}

void FOnlineUserCloudInterfaceIOS::OnDeleteUserCloudFileBegin(const FString &  FileName)
{
	OnDeleteUserCloudFileCompleteDelegateHandle = AddOnDeleteUserFileCompleteDelegate_Handle(OnDeleteUserCloudFileCompleteDelegate);

	DeleteUserFile(*UniqueNetId, FileName, true, true); 
}
