// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineUserCloudInterface.h"
#include "OnlineSubsystemIOSTypes.h"
#include "IOSSaveGameSystem.h"

#import <CloudKit/CloudKit.h>

@class IOSCloudStorage;

@interface IOSCloudStorage : NSObject

@property(retain) CKContainer* CloudContainer;
@property(retain) CKDatabase* SharedDatabase;
@property(retain) CKDatabase* UserDatabase;
@property(retain) id iCloudToken;

-(IOSCloudStorage*)init:(bool)registerHandler;
-(bool)readFile:(NSString*)fileName sharedDB:(bool)shared completionHandler:(void(^)(CKRecord* record, NSError* error))handler;
-(bool)writeFile:(NSString*)fileName contents:(NSData*)fileContents sharedDB:(bool)shared completionHandler:(void(^)(CKRecord* record, NSError* error))handler;
-(bool)deleteFile:(NSString*)fileName sharedDB:(bool)shared completionHandler:(void(^)(CKRecordID* record, NSError* error))handler;
-(bool)query:(bool)shared fetchHandler : (void(^)(CKRecord* record))fetch completionHandler : (void(^)(CKQueryCursor* record, NSError* error))complete;

-(void)iCloudAccountAvailabilityChanged:(NSNotification*)notification;

+(IOSCloudStorage*)cloudStorage;

@end

/**
*	FOnlineUserCloudInterfaceIOS - Implementation of user cloud storage for IOS
*/
class FOnlineUserCloudInterfaceIOS : public IOnlineUserCloud
{
protected:

	FCloudFile* GetCloudFile(const FString& FileName, bool bCreateIfMissing = false);
    FCloudFileHeader* GetCloudFileHeader(const FString& FileName, bool bCreateIfMissing = false);
	bool ClearFiles();
	bool ClearCloudFile(const FString& FileName);

public:
    FOnlineUserCloudInterfaceIOS() : SaveSystem(NULL), UpdateDictionary(NULL) { MetaDataState = EOnlineAsyncTaskState::Done; }
	virtual ~FOnlineUserCloudInterfaceIOS();

	// IOnlineUserCloud
	virtual bool GetFileContents(const FUniqueNetId& UserId, const FString& FileName, TArray<uint8>& FileContents) override;
	virtual bool ClearFiles(const FUniqueNetId& UserId) override;
	virtual bool ClearFile(const FUniqueNetId& UserId, const FString& FileName) override;
	virtual void EnumerateUserFiles(const FUniqueNetId& UserId) override;
	virtual void GetUserFileList(const FUniqueNetId& UserId, TArray<FCloudFileHeader>& UserFiles) override;
	virtual bool ReadUserFile(const FUniqueNetId& UserId, const FString& FileName) override;
	virtual bool WriteUserFile(const FUniqueNetId& UserId, const FString& FileName, TArray<uint8>& FileContents, bool bCompressBeforeUpload = false) override;
	virtual void CancelWriteUserFile(const FUniqueNetId& UserId, const FString& FileName) override;
	virtual bool DeleteUserFile(const FUniqueNetId& UserId, const FString& FileName, bool bShouldCloudDelete, bool bShouldLocallyDelete) override;
	virtual bool RequestUsageInfo(const FUniqueNetId& UserId) override;
	virtual void DumpCloudState(const FUniqueNetId& UserId) override;
	virtual void DumpCloudFileState(const FUniqueNetId& UserId, const FString& FileName) override;

	// Initialize Cloud saving
	void InitCloudSave(bool InIOSAlwaysSyncCloudFiles);
private:
	/** File metadata */
	TArray<struct FCloudFileHeader> CloudMetaData;
    /** File metadata query state */
    EOnlineAsyncTaskState::Type MetaDataState;
	/** File cache */
	TArray<struct FCloudFile> CloudFileData;

    /** Critical section for thread safe operation on cloud files */
    FCriticalSection CloudDataLock;

	/** Reference to the iOS file save system */
	FIOSSaveGameSystem* SaveSystem;

	/** Flag from Settings->iOS: Always read from the iCloud on LoadGame */
	bool bIOSAlwaysSyncCloudFiles;

	/** Store the iCloud sync status for each save file
	* entris of type (string) filename: (bool) synced with iCloud
	* updated by the silent notifications, if enabled
	*/
	NSMutableDictionary* UpdateDictionary;
	
	/** Delegates to various cloud functionality triggered */
	FOnEnumerateUserFilesCompleteDelegate OnEnumerateUserCloudFilesCompleteDelegate;
	FOnReadUserFileCompleteDelegate OnInitialFetchUserCloudFileCompleteDelegate;
	FOnWriteUserFileCompleteDelegate OnWriteUserCloudFileCompleteDelegate;
	FOnReadUserFileCompleteDelegate OnReadUserCloudFileCompleteDelegate;
	FOnDeleteUserFileCompleteDelegate OnDeleteUserCloudFileCompleteDelegate;

	/** Handles to those delegates */
	FDelegateHandle OnEnumerateUserCloudFilesCompleteDelegateHandle;
	FDelegateHandle OnWriteUserCloudFileCompleteDelegateHandle;
	FDelegateHandle OnReadUserCloudFileCompleteDelegateHandle;
	FDelegateHandle OnDeleteUserCloudFileCompleteDelegateHandle;
	
	/** Cache the UserId */
	FUniqueNetIdIOSPtr UniqueNetId;

	/**
	 *	Delegate triggered when all user files have been enumerated
	 * @param bWasSuccessful did the operation complete successfully
	 * @param UserId user that triggered the operation
	 */
	void OnEnumerateUserFilesComplete(bool bWasSuccessful, const FUniqueNetId& UserId);

	/**
	 *	Delegate triggered on the init for each user cloud file read - will overwrite the local files
	 * @param bWasSuccessful did the operation complete successfully
	 * @param UserId user that triggered the operation
	 * @param FileName filename read from cloud
	 */
	void OnInitialFetchUserCloudFileComplete(bool bWasSuccessful, const FUniqueNetId& UserId, const FString& FileName);

	/**
	 *	Delegate triggered for each user cloud file written
	 * @param bWasSuccessful did the operation complete successfully
	 * @param UserId user that triggered the operation
	 * @param FileName filename written to cloud
	 */
	void OnWriteUserCloudFileComplete(bool bWasSuccessful, const FUniqueNetId& UserId, const FString& FileName);

	/**
	 *	Delegate triggered for each user cloud file read - will overwrite the local files
	 * @param bWasSuccessful did the operation complete successfully
	 * @param UserId user that triggered the operation
	 * @param FileName filename read from cloud
	 */
	void OnReadUserCloudFileComplete(bool bWasSuccessful, const FUniqueNetId& UserId, const FString& FileName);

	/**
	 *	Delegate triggered for each user cloud file deleted
	 * @param bWasSuccessful did the operation complete successfully
	 * @param UserId user that triggered the operation
	 * @param FileName filename deleted from cloud
	 */
	void OnDeleteUserCloudFileComplete(bool bWasSuccessful, const FUniqueNetId& UserId, const FString& FileName);

	/**
	 *	Delegate in the iOS file save system
	 * Called in SaveGame
	 */
	bool OnWriteUserCloudFileBegin(const FString&  FileName);

	/**
	 *	Delegate in the iOS file save system
	 * Called in ReadGame
	 */
	bool OnReadUserCloudFileBegin(const FString & FileName);

	/**
	 *	Delegate in the iOS file save system
	 * Called in DeleteGame
	 */
	void OnDeleteUserCloudFileBegin(const FString&  FileName);

	/**
	 *	Returns true if the record must be fetched from the iCloud
	 */
	 bool ShouldFetchRecordFromCloud(const FString &  FileName);

};

typedef TSharedPtr<FOnlineUserCloudInterfaceIOS, ESPMode::ThreadSafe> FOnlineUserCloudIOSPtr;
