// Copyright Epic Games, Inc. All Rights Reserved.

import Foundation
import StoreKit

@objcMembers class StoreKit2 : NSObject
{
	static func testRefund(_ transactionId: UInt64, inScene: UIWindowScene)
	{
		Task
		{
#if os(tvOS)
			print("Refund is not supported on tvOS.")
#else
			let status = try await Transaction.beginRefundRequest(for: transactionId, in:inScene)
			print("Refund status: \(status)")
#endif
		}
	}

	static func getCountry(_ completionHandler: @escaping (String) -> Void) 
	{
		Task 
		{
			let countryCode: String
        
			if let storefront = await Storefront.current 
			{
				countryCode = storefront.countryCode
			} 
			else 
			{
				countryCode = ""
			}

			DispatchQueue.main.async 
			{
				completionHandler(countryCode)
			}
		}
	}
}

