// Copyright Epic Games, Inc. All Rights Reserved.

#include "TurnBasedEventListener.h"

@interface FTurnBasedEventListenerIOS()
{
    FTurnBasedEventDelegate* _owner;
}

@end

@implementation FTurnBasedEventListenerIOS

- (id)initWithOwner:(FTurnBasedEventDelegate&)owner
{
    if (self = [super init]) {
        _owner = &owner;
        [[GKLocalPlayer localPlayer] registerListener:self];
    }
    return self;
}

- (void)dealloc
{
    [[GKLocalPlayer localPlayer] unregisterListener:self];
    [super dealloc];
}

// GKChallengeListener
- (void)player:(GKPlayer *)player didCompleteChallenge:(GKChallenge *)challenge issuedByFriend:(GKPlayer *)friendPlayer {}
- (void)player:(GKPlayer *)player didReceiveChallenge:(GKChallenge *)challenge {}
- (void)player:(GKPlayer *)player issuedChallengeWasCompleted:(GKChallenge *)challenge byFriend:(GKPlayer *)friendPlayer {}
- (void)player:(GKPlayer *)player wantsToPlayChallenge:(GKChallenge *)challenge {}

// GKInviteEventListener
- (void)player:(GKPlayer *)player didAcceptInvite:(GKInvite *)invite {}

// GKInviteEventListener and GKTurnBasedEventListener
- (void)player:(GKPlayer *)player didRequestMatchWithPlayers:(NSArray *)playerIDsToInvite {}

// GKTurnBasedEventListener
- (void)player:(GKPlayer *)player matchEnded:(GKTurnBasedMatch *)match
{
    if (_owner) {
        _owner->OnMatchEnded([self getMatchIDFromMatch:match]);
    }
}

- (void)player:(GKPlayer *)player receivedExchangeCancellation:(GKTurnBasedExchange *)exchange forMatch:(GKTurnBasedMatch *)match {}
- (void)player:(GKPlayer *)player receivedExchangeReplies:(NSArray *)replies forCompletedExchange:(GKTurnBasedExchange *)exchange forMatch:(GKTurnBasedMatch *)match {}
- (void)player:(GKPlayer *)player receivedExchangeRequest:(GKTurnBasedExchange *)exchange forMatch:(GKTurnBasedMatch *)match {}

- (void)player:(GKPlayer *)player receivedTurnEventForMatch:(GKTurnBasedMatch *)match didBecomeActive:(BOOL)didBecomeActive
{
    if (_owner) {
       _owner->OnMatchReceivedTurnEvent([self getMatchIDFromMatch:match], didBecomeActive, match);
    }
}

- (FString)getMatchIDFromMatch:(GKTurnBasedMatch *)match
{
    return FString::Printf(TEXT("%s"), UTF8_TO_TCHAR(match.matchID.UTF8String));
}

@end
