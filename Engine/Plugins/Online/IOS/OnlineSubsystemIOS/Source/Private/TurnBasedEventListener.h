// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <GameKit/GameKit.h>
#include "Interfaces/OnlineTurnBasedInterface.h"

@interface FTurnBasedEventListenerIOS : NSObject < GKLocalPlayerListener >

- (id)initWithOwner:(FTurnBasedEventDelegate&)owner;

// GKChallengeListener
- (void)player:(GKPlayer *)player didCompleteChallenge:(GKChallenge *)challenge issuedByFriend:(GKPlayer *)friendPlayer;
- (void)player:(GKPlayer *)player didReceiveChallenge:(GKChallenge *)challenge;
- (void)player:(GKPlayer *)player issuedChallengeWasCompleted:(GKChallenge *)challenge byFriend:(GKPlayer *)friendPlayer;
- (void)player:(GKPlayer *)player wantsToPlayChallenge:(GKChallenge *)challenge;

// GKInviteEventListener
- (void)player:(GKPlayer *)player didAcceptInvite:(GKInvite *)invite;

// GKInviteEventListener and GKTurnBasedEventListener
-(void)player:(GKPlayer *)player didRequestMatchWithPlayers:(NSArray *)playerIDsToInvite;

// GKTurnBasedEventListener
- (void)player:(GKPlayer *)player matchEnded:(GKTurnBasedMatch *)match;
- (void)player:(GKPlayer *)player receivedExchangeCancellation:(GKTurnBasedExchange *)exchange forMatch:(GKTurnBasedMatch *)match;
- (void)player:(GKPlayer *)player receivedExchangeReplies:(NSArray *)replies forCompletedExchange:(GKTurnBasedExchange *)exchange forMatch:(GKTurnBasedMatch *)match;
- (void)player:(GKPlayer *)player receivedExchangeRequest:(GKTurnBasedExchange *)exchange forMatch:(GKTurnBasedMatch *)match;
- (void)player:(GKPlayer *)player receivedTurnEventForMatch:(GKTurnBasedMatch *)match didBecomeActive:(BOOL)didBecomeActive;

@end
