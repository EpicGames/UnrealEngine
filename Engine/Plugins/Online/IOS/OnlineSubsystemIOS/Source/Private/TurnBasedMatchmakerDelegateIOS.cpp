// Copyright Epic Games, Inc. All Rights Reserved.

#include "TurnBasedMatchmakerDelegateIOS.h"
#include "OnlineTurnBasedInterfaceIOS.h"
#include "Interfaces/OnlineTurnBasedInterface.h"
#include "OnlineSubsystemIOS.h"
#include <GameKit/GKLocalPlayer.h>

@interface FTurnBasedMatchmakerDelegateIOS()
{
	FTurnBasedMatchmakerDelegate* _delegate;
}

@end

@implementation FTurnBasedMatchmakerDelegateIOS

- (id)initWithDelegate:(FTurnBasedMatchmakerDelegate*)delegate
{
	if (self = [super init])
	{
		_delegate = delegate;
	}
	return self;
}

-(void)turnBasedMatchmakerViewController:(GKTurnBasedMatchmakerViewController*)viewController didFailWithError : (NSError*)error
{
	if (_delegate) {
		_delegate->OnMatchmakerFailed();
	}
}

-(void)turnBasedMatchmakerViewController:(GKTurnBasedMatchmakerViewController*)viewController didFindMatch : (GKTurnBasedMatch*)match
{
	if (_delegate) {
		[match loadMatchDataWithCompletionHandler : ^ (NSData* matchData, NSError* matchLoadError) {
			if (matchLoadError) {
				[self turnBasedMatchmakerViewController : viewController didFailWithError : matchLoadError];
				return;
			}

			NSMutableArray* playerIdentifierArray = [NSMutableArray array];
			for (GKTurnBasedParticipant* participant in match.participants) {
				NSString* PlayerIDString = nil;
				if ([GKTurnBasedParticipant respondsToSelector:@selector(player)] == YES)
				{
					PlayerIDString = FOnlineSubsystemIOS::GetPlayerId(participant.player);
				}
				if (!PlayerIDString)
				{
					break;
				}
				[playerIdentifierArray addObject : PlayerIDString];
			}
			
			GKLocalPlayer* GKLocalUser = [GKLocalPlayer localPlayer];
			[GKLocalUser loadFriendsWithIdentifiers:(NSArray<NSString *> *)playerIdentifierArray completionHandler:^(NSArray<GKPlayer *> *players, NSError *nameLoadError)
			{
				if (nameLoadError)
				{
					[self turnBasedMatchmakerViewController : viewController didFailWithError : nameLoadError];
					return;
				}

				FTurnBasedMatchRef MatchRef(new FTurnBasedMatchIOS(match, players));
				_delegate->OnMatchFound(MatchRef);
			}];
		}];
	}
	else {
		[match removeWithCompletionHandler : nil];
	}
}

-(void)turnBasedMatchmakerViewController:(GKTurnBasedMatchmakerViewController*)viewController playerQuitForMatch : (GKTurnBasedMatch*)match
{
}

-(void)turnBasedMatchmakerViewControllerWasCancelled : (GKTurnBasedMatchmakerViewController*)viewController
{
	if (_delegate) {
		_delegate->OnMatchmakerCancelled();
	}
}

@end
