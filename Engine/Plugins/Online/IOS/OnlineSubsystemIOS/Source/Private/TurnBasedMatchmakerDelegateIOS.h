// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <GameKit/GameKit.h>
#include "Interfaces/OnlineTurnBasedInterface.h"

@interface FTurnBasedMatchmakerDelegateIOS : NSObject < GKTurnBasedMatchmakerViewControllerDelegate >

- (id)initWithDelegate:(FTurnBasedMatchmakerDelegate*)delegate;

- (void)turnBasedMatchmakerViewController:(GKTurnBasedMatchmakerViewController*)viewController didFailWithError:(NSError*)error;
- (void)turnBasedMatchmakerViewController:(GKTurnBasedMatchmakerViewController*)viewController didFindMatch:(GKTurnBasedMatch*)match;
- (void)turnBasedMatchmakerViewController:(GKTurnBasedMatchmakerViewController*)viewController playerQuitForMatch:(GKTurnBasedMatch*)match;
- (void)turnBasedMatchmakerViewControllerWasCancelled:(GKTurnBasedMatchmakerViewController*)viewController;

@end
