// Copyright Epic Games, Inc. All Rights Reserved.

#include "TurnBasedMatchmakerIOS.h"

#include <GameKit/GameKit.h>
#include "IOS/IOSAppDelegate.h"
#include "TurnBasedMatchmakerDelegateIOS.h"
#include "IOS/IOSView.h"

DEFINE_LOG_CATEGORY_STATIC(LogTurnBasedMatchmakerIOS, Log, All);

FTurnBasedMatchmakerIOS::FTurnBasedMatchmakerIOS(FTurnBasedMatchmakerDelegate& _Delegate)
: MatchmakerViewController(nil)
, IOSDelegate(nil)
, Delegate(&_Delegate)
{
}

FTurnBasedMatchmakerIOS::~FTurnBasedMatchmakerIOS()
{
    [IOSDelegate release];
    [MatchmakerViewController release];
}

void FTurnBasedMatchmakerIOS::ShowWithMatchRequest(const FTurnBasedMatchRequest& Request)
{
    MatchRequest = Request;

    IOSDelegate = [[FTurnBasedMatchmakerDelegateIOS alloc] initWithDelegate:this];

    GKMatchRequest* GKRequest = GetGKMatchRequestFromMPMatchRequest(Request);
    MatchmakerViewController = [[GKTurnBasedMatchmakerViewController alloc] initWithMatchRequest:GKRequest];

    if (MatchmakerViewController)
    {
        MatchmakerViewController.showExistingMatches = Request.GetShowExistingMatches() ? YES : NO;
        MatchmakerViewController.turnBasedMatchmakerDelegate = IOSDelegate;
        UIViewController* MainViewController = [IOSAppDelegate GetDelegate].IOSController;
        [MainViewController presentViewController:MatchmakerViewController animated:YES completion:nil];
    }
}

void FTurnBasedMatchmakerIOS::OnMatchmakerCancelled()
{
    UE_LOGF(LogTurnBasedMatchmakerIOS, Log, "Matchmaker cancelled");

    if (Delegate)
    {
        Delegate->OnMatchmakerCancelled();
    }
    DismissMatchmaker();
}

void FTurnBasedMatchmakerIOS::OnMatchmakerFailed()
{
    UE_LOGF(LogTurnBasedMatchmakerIOS, Log, "Matchmaker failed");

    if (Delegate)
    {
        Delegate->OnMatchmakerFailed();
    }
    DismissMatchmaker();
}

void FTurnBasedMatchmakerIOS::OnMatchFound(FTurnBasedMatchRef Match)
{
    UE_LOGF(LogTurnBasedMatchmakerIOS, Log, "Match found");
    //Match->SetMatchRequest(MatchRequest);

    if (Delegate)
    {
        Delegate->OnMatchFound(Match);
    }

    DismissMatchmaker();
}

void FTurnBasedMatchmakerIOS::DismissMatchmaker()
{
    if (!MatchmakerViewController)
    {
        UE_LOGF(LogTurnBasedMatchmakerIOS, Warning, "No matchmaker was active");
        return;
    }

    [IOSDelegate release];
    IOSDelegate = nil;

    [MatchmakerViewController dismissViewControllerAnimated:YES completion:nil];
    [MatchmakerViewController release];
    MatchmakerViewController = nil;
}

GKMatchRequest* FTurnBasedMatchmakerIOS::GetGKMatchRequestFromMPMatchRequest(const FTurnBasedMatchRequest& Request) const
{
    GKMatchRequest* NewMatchRequest = [[GKMatchRequest alloc] init];

    NewMatchRequest.maxPlayers = Request.GetMaxNumberOfPlayers();
    NewMatchRequest.minPlayers = Request.GetMinNumberOfPlayers();
    NewMatchRequest.defaultNumberOfPlayers = Request.GetMinNumberOfPlayers();
    NewMatchRequest.playerGroup = Request.GetPlayerGroup();

    return [NewMatchRequest autorelease];
}
