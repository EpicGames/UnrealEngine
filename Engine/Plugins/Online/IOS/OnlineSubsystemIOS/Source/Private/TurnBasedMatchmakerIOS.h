// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineTurnBasedInterface.h"

@class FTurnBasedMatchmakerDelegateIOS;
@class GKMatchRequest;
@class GKTurnBasedMatchmakerViewController;

class FTurnBasedMatchmakerIOS : public FTurnBasedMatchmakerDelegate
{
public:
    FTurnBasedMatchmakerIOS(FTurnBasedMatchmakerDelegate& _Delegate);
    ~FTurnBasedMatchmakerIOS();

    void ShowWithMatchRequest(const FTurnBasedMatchRequest& Request);

    virtual void OnMatchmakerCancelled() override;
    virtual void OnMatchmakerFailed() override;
    virtual void OnMatchFound(FTurnBasedMatchRef Match) override;

private:
    GKMatchRequest* GetGKMatchRequestFromMPMatchRequest(const FTurnBasedMatchRequest& Request) const;
    void DismissMatchmaker();

    FTurnBasedMatchRequest MatchRequest;

    GKTurnBasedMatchmakerViewController* MatchmakerViewController;
    FTurnBasedMatchmakerDelegateIOS* IOSDelegate;
    FTurnBasedMatchmakerDelegate* Delegate;
};
