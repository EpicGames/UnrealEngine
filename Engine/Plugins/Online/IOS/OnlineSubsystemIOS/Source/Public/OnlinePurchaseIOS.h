// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_WITH_STORE_KIT

#include "Interfaces/OnlinePurchaseInterface.h"
#include "RetainedObjCInstance.h"

enum class EPurchaseTransactionState : uint8;
struct FOnlinePurchaseTransactionIOS;
struct FOnlinePurchaseInProgressTransactionIOS;
@class SKPaymentTransaction;
@class FStoreKitPurchaseProxy;

/**
 * Implementation for online purchase via IOS services
 */
class FOnlinePurchaseIOS :
	public IOnlinePurchase,
	public TSharedFromThis<FOnlinePurchaseIOS, ESPMode::ThreadSafe>
{

public:

	// IOnlinePurchase

	virtual bool IsAllowedToPurchase(const FUniqueNetId& UserId) override;
	virtual bool IsAllowedToPurchase(const FUniqueNetId& UserId, FOnlineError& Error) override;
	virtual void Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseCheckoutComplete& Delegate) override;
	virtual void Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseReceiptlessCheckoutComplete& Delegate) override;
	virtual void FinalizePurchase(const FUniqueNetId& UserId, const FString& ReceiptId) override;
	virtual void RedeemCode(const FUniqueNetId& UserId, const FRedeemCodeRequest& RedeemCodeRequest, const FOnPurchaseRedeemCodeComplete& Delegate) override;
	virtual void QueryReceipts(const FUniqueNetId& UserId, bool bRestoreReceipts, const FOnQueryReceiptsComplete& Delegate) override;
	virtual void GetReceipts(const FUniqueNetId& UserId, TArray<FPurchaseReceipt>& OutReceipts) const override;
	virtual void FinalizeReceiptValidationInfo(const FUniqueNetId& UserId, FString& InReceiptValidationInfo, const FOnFinalizeReceiptValidationInfoComplete& Delegate) override;
	
#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
	virtual void RequestExternalPurchaseToken(const FOnRequestExternalPurchaseTokenComplete& Delegate) const override;
#endif // !PLATFORM_TVOS && !PLATFORM_VISIONOS


	// FOnlinePurchaseIOS

	/**
	 * Constructor
	 *
	 * @param InSubsystem IOS subsystem being used
	 */
	FOnlinePurchaseIOS(class FOnlineSubsystemIOS* InSubsystem);

	/**
	 * Destructor
	 */
	virtual ~FOnlinePurchaseIOS();
	
	/**
	 * Log App Receipt content
	 */
	void DumpAppReceipt();
	
	/**
	 * Asynchronously retrieves the App Store receipt data
	 */
	void GetAppReceipt(TFunction<void(const FString&)> OnComplete);
	
	/**
	 * Method used internally by FStoreKitPurchaseProxy to handle a known transactions. Not meant to be called by user code
	 */
	void OnUpdatedTransactions(const TArray<FOnlinePurchaseTransactionIOS>& Transactions);
	
	/**
	 * Method used internally by FStoreKitPurchaseProxy to handle finalized transactions. Not meant to be called by user code
	 */
	void OnRemovedTransactions(const TArray<FOnlinePurchaseTransactionIOS>& Transactions);
	
    /**
     * Method used internally by FStoreKitPurchaseProxy to notify that transactions were restored. Not meant to be called by user code
     */
    void OnQueryReceiptsComplete(bool bSuccess, const TSharedPtr<TArray<FOnlinePurchaseTransactionIOS>>& Transactions);
private:
	/** Entry data for a known completed transaction (restored or purchased) */
	struct FKnownTransaction
	{
		explicit FKnownTransaction(TSharedRef<FPurchaseReceipt> InReceipt, SKPaymentTransaction* InTransaction)
		: Receipt(InReceipt)
		, PaymentTransaction(InTransaction)
		{
		}

		TSharedRef<FPurchaseReceipt> Receipt;
		TRetainedObjCInstance<SKPaymentTransaction*> PaymentTransaction;
	};

	/**
     * Add receipt information for transaction data to intermediate cache avoiding duplicates and return stored receipt
     */
	static void AddReceiptToCache(TArray<FKnownTransaction>& Cache, const TSharedRef<FPurchaseReceipt>& Receipt, const FOnlinePurchaseTransactionIOS& Transaction);

    /**
     * Generate a receipt for a failed purchase
     */
    static TSharedRef<FPurchaseReceipt> GenerateFailReceipt(const FPurchaseCheckoutRequest& CheckoutRequest);
    
    /**
     * Generate a receipt for a successful purchase which was received offline
     */
    static TSharedRef<FPurchaseReceipt> GenerateOfflineReceipt(const FOnlinePurchaseTransactionIOS& Transaction);

    /**
     * Generate a final receipt for the purchase currently in progress
     */
    static TSharedRef<FPurchaseReceipt> GenerateReceipt(const FPurchaseCheckoutRequest& CheckoutRequest, const FOnlinePurchaseTransactionIOS& Transaction);
private:
	
	/** Proxy helper to communicate from/to StoreKit*/
    TRetainedObjCInstance<FStoreKitPurchaseProxy*> StoreKitProxy;
	
	/** Are receipts being queried */
	bool bQueryingReceipts;

	/** Transient delegate to fire when restoring transactions (QueryReceipts with bRestoreReceipts=true)*/
	FOnQueryReceiptsComplete QueryReceiptsComplete;
	
	/** Keeps track of in progress user transaction */
    TSharedPtr<const FOnlinePurchaseInProgressTransactionIOS> InProgressTransaction;
	
	/** Cache of known transactions */
	TArray<FKnownTransaction> CachedReceipts;
	
	/** Transactions in the process of being finalized */
	TArray<FKnownTransaction> FinalizingTransactions;

	/** Reference to the parent subsystem */
	FOnlineSubsystemIOS* Subsystem;
	
	/** Delegate fired when a transaction is finalized */
	using FOnTransactionFinalized = TMulticastDelegate<void(const FOnlinePurchaseTransactionIOS&)>;
	FOnTransactionFinalized OnTransactionFinalized;
};

typedef TSharedPtr<FOnlinePurchaseIOS, ESPMode::ThreadSafe> FOnlinePurchaseIOSPtr;

#endif //UE_WITH_STORE_KIT
