// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_WITH_STORE_KIT

#include "Interfaces/OnlineStoreInterfaceV2.h"
#import <Foundation/Foundation.h>
#include "RetainedObjCInstance.h"

enum class EPurchaseTransactionState : uint8;
@class FStoreKitStoreProxy;
@class SKProductsResponse;
@class SKProduct;

/**
 * Implementation for online store via IOS services
 */
class FOnlineStoreIOS :
	public IOnlineStoreV2,
	public TSharedFromThis<FOnlineStoreIOS, ESPMode::ThreadSafe>
{
public:

	// IOnlineStoreV2
    
	virtual void QueryCategories(const FUniqueNetId& UserId, const FOnQueryOnlineStoreCategoriesComplete& Delegate) override;
	virtual void GetCategories(TArray<FOnlineStoreCategory>& OutCategories) const override;
	virtual void QueryOffersByFilter(const FUniqueNetId& UserId, const FOnlineStoreFilter& Filter, const FOnQueryOnlineStoreOffersComplete& Delegate) override;
	virtual void QueryOffersById(const FUniqueNetId& UserId, const TArray<FUniqueOfferId>& OfferIds, const FOnQueryOnlineStoreOffersComplete& Delegate) override;
	virtual void GetOffers(TArray<FOnlineStoreOfferRef>& OutOffers) const override;
	virtual TSharedPtr<FOnlineStoreOffer> GetOffer(const FUniqueOfferId& OfferId) const override;
	virtual FString GetStoreCountry() const override;

	// FOnlineStoreIOS

	/**
	 * Constructor
	 *
	 * @param InSubsystem Online subsystem being used
	 */
	FOnlineStoreIOS(class FOnlineSubsystemIOS* InSubsystem);

	/**
	 * Destructor
	 */
	virtual ~FOnlineStoreIOS();
	
	/**
	 * Get the production information for a given offer id
	 * Must have previously been retrieved by QueryOffers*
	 *
	 * @return the SKProduct previously queried for product information
	 */
	SKProduct* GetSKProductByOfferId(const FUniqueOfferId& OfferId);

    /**
     * Method used internally by FOnlineSubsystemIOS to notify that QueryOffers request finished. Not meant to be called by user code
     */
    void OnProductsRequestResponse(SKProductsResponse* Response, const FOnQueryOnlineStoreOffersComplete& CompletionDelegate);

	/**
	 * Starts an async request to get the store kit country
	 */
	void QueryStoreCountry();
	

private:
	
	/**
	 * Representation of a single product offer
	 */
    struct FOnlineStoreOfferIOS
	{
		FOnlineStoreOfferIOS()
		: FOnlineStoreOfferIOS(nil, nullptr)
		{
		}
		
		FOnlineStoreOfferIOS(SKProduct* InProduct, TSharedPtr<FOnlineStoreOffer>&& InOffer)
		{
			Product = InProduct;
			Offer = MoveTemp(InOffer);
		}
		
		/** @return true if the store offer is valid/proper */
		bool IsValid() const
		{
			return Product != nil && Offer.IsValid();
		}
		
		/** Reference to the app store product information */
		TRetainedObjCInstance<SKProduct*> Product;
		/** Product information about this offer */
		TSharedPtr<FOnlineStoreOffer> Offer;
	};

	void AddOffer(const FOnlineStoreOfferIOS& NewOffer);

    bool OffersNotAllowedInLocale(const FString& Locale);
    
private:
	
	/** Mapping from offer id to product information */
	typedef TMap<FUniqueOfferId, FOnlineStoreOfferIOS> FOnlineOfferDescriptionMap;

	/** Mapping of all queried offers to their product information */
	FOnlineOfferDescriptionMap CachedOffers;

	/** Store kit helper for interfacing with app store */
    TRetainedObjCInstance<FStoreKitStoreProxy*> StoreKitProxy;

	/** Is a query already in flight */
	bool bIsQueryInFlight;
	
	FString StoreCountry;
	TAtomic<bool> bIsStoreCountryInitialized = false;

	/** Reference to the parent subsystem */
	FOnlineSubsystemIOS* Subsystem;
};

typedef TSharedPtr<FOnlineStoreIOS, ESPMode::ThreadSafe> FOnlineStoreIOSPtr;
#endif // UE_WITH_STORE_KIT
