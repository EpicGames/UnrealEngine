// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystem.h"
#include "OnlineSubsystemImpl.h"
#include "OnlineSubsystemIOSPackage.h"
#include "OnlineSessionInterfaceIOS.h"
#include "OnlineFriendsInterfaceIOS.h"
#include "OnlineIdentityInterfaceIOS.h"
#include "OnlineLeaderboardsInterfaceIOS.h"
#include "OnlineStoreIOS.h"
#include "OnlinePurchaseIOS.h"
#include "OnlineAchievementsInterfaceIOS.h"
#include "OnlineExternalUIInterfaceIOS.h"
#include "OnlineTurnBasedInterfaceIOS.h"
#include "OnlineUserCloudInterfaceIOS.h"
#include "OnlineSharedCloudInterfaceIOS.h"

@class FAppStoreUtils;
@class GKPlayer;
@class GKLocalPlayer;


/**
 *	OnlineSubsystemIOS - Implementation of the online subsystem for IOS services
 */
class ONLINESUBSYSTEMIOS_API FOnlineSubsystemIOS : 
	public FOnlineSubsystemImpl
{

public:
	
	virtual ~FOnlineSubsystemIOS() = default;

	//~ Begin IOnlineSubsystem Interface
	virtual IOnlineSessionPtr GetSessionInterface() const override;
	virtual IOnlineFriendsPtr GetFriendsInterface() const override;
	virtual IOnlinePartyPtr GetPartyInterface() const override;
	virtual IOnlineGroupsPtr GetGroupsInterface() const override;
	virtual IOnlineSharedCloudPtr GetSharedCloudInterface() const override;
	virtual IOnlineUserCloudPtr GetUserCloudInterface() const override;
	virtual IOnlineLeaderboardsPtr GetLeaderboardsInterface() const override;
	virtual IOnlineVoicePtr GetVoiceInterface() const  override;
	virtual IOnlineExternalUIPtr GetExternalUIInterface() const override;
	virtual IOnlineTimePtr GetTimeInterface() const override;
	virtual IOnlineIdentityPtr GetIdentityInterface() const override;
	virtual IOnlineTitleFilePtr GetTitleFileInterface() const override;
	virtual IOnlineEntitlementsPtr GetEntitlementsInterface() const override;
	virtual IOnlineStoreV2Ptr GetStoreV2Interface() const override;
	virtual IOnlinePurchasePtr GetPurchaseInterface() const override;
	virtual IOnlineEventsPtr GetEventsInterface() const override;
	virtual IOnlineAchievementsPtr GetAchievementsInterface() const override;
	virtual IOnlineSharingPtr GetSharingInterface() const override;
	virtual IOnlineUserPtr GetUserInterface() const override;
	virtual IOnlineMessagePtr GetMessageInterface() const override;
	virtual IOnlinePresencePtr GetPresenceInterface() const override;
	virtual IOnlineChatPtr GetChatInterface() const override;
	virtual IOnlineStatsPtr GetStatsInterface() const override;
	virtual IOnlineTurnBasedPtr GetTurnBasedInterface() const override;
	virtual IOnlineTournamentPtr GetTournamentInterface() const override;
	virtual bool Init() override;
	virtual bool Shutdown() override;
	virtual FString GetAppId() const override;
	virtual bool Exec(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	virtual bool Tick(float DeltaTime) override;
	virtual FText GetOnlineServiceName() const override;
	virtual bool IsEnabled() const override;
	//~ End IOnlineSubsystem Interface

PACKAGE_SCOPE:

	/** Only the factory makes instances */
	FOnlineSubsystemIOS() = delete;
	explicit FOnlineSubsystemIOS(FName InInstanceName);

	/**
	 * Is IAP available for use
	 * @return true if IAP should be available, false otherwise
	 */
	static bool IsInAppPurchasingEnabled();
	
	/**
	 * Is GameCenter enabled
	 * @return true if enabled, false otherwise
	 */
	static bool IsGameCenterEnabled();
	
	/**
	 * Is CloudKit enabled
	 * @return true if enabled, false otherwise
	 */
	static bool IsCloudKitEnabled();
		
	/**
	 * @return access to the app store utility class
	 */
	FAppStoreUtils* GetAppStoreUtils();

public:
	static NSString* GetPlayerId(GKPlayer* Player);
	static NSString* GetPlayerId(GKLocalPlayer* Player);

private:
	
	void InitAppStoreHelper();
	void CleanupAppStoreHelper();

	/** Handle purchase command cheat codes */
	bool HandlePurchaseExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);

	/** Online async task thread */
	class FRunnableThread* OnlineAsyncTaskThread;
	
	/** App store util helper */
	FAppStoreUtils* AppStoreHelper;

	/** Interface to the session services */
	FOnlineSessionIOSPtr SessionInterface;

	/** Interface to the Identity information */
	FOnlineIdentityIOSPtr IdentityInterface;

	/** Interface to the friends services */
	FOnlineFriendsIOSPtr FriendsInterface;

	/** Interface to the profile information */
	FOnlineLeaderboardsIOSPtr LeaderboardsInterface;
	
#if UE_WITH_STORE_KIT
	/** Interface to the online catalog */
	FOnlineStoreIOSPtr StoreV2Interface;
	
	/** Interface to the store purchasing */
	FOnlinePurchaseIOSPtr PurchaseInterface;
#endif
	/** Interface to the online achievements */
	FOnlineAchievementsIOSPtr AchievementsInterface;

	/** Interface to the external UI services */
	FOnlineExternalUIIOSPtr ExternalUIInterface;

    /** Interface to the turnbased multiplayer services */
    FOnlineTurnBasedIOSPtr TurnBasedInterface;

	/** Interface to the user cloud storage */
	FOnlineUserCloudIOSPtr UserCloudInterface;

	/** Interface to the shared cloud storage */
	FOnlineSharedCloudIOSPtr SharedCloudInterface;
};

typedef TSharedPtr<FOnlineSubsystemIOS, ESPMode::ThreadSafe> FOnlineSubsystemIOSPtr;

