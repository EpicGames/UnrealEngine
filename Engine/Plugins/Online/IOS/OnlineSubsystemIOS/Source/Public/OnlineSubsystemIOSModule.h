// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

/**
 * Online subsystem module class (GameCenter Implementation)
 * Code related to the loading and handling of the GameCenter module.
 */
class FOnlineSubsystemIOSModule : public IModuleInterface
{
private:

	/** Class responsible for creating instance(s) of the subsystem */
	class FOnlineFactoryIOS* IOSFactory;

public:

	FOnlineSubsystemIOSModule() :
		IOSFactory(NULL)
	{
	}

	virtual ~FOnlineSubsystemIOSModule(){}

	//~ Begin IModuleInterface Interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

	virtual bool SupportsAutomaticShutdown() override
	{
		return false;
	}
	//~ End IModuleInterface Interface
};


