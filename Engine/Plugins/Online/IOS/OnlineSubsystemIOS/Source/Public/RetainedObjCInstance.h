// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include <objc/NSObject.h>

/**
 * RAII wrapper type that encapsulates manual memory management for Objective-C instances:
 * - It calls retain when a non nil instance is received
 * - It calls release on destruction
 * - It handles copy/move semantics to keep reference counting consistent
 * This type is useful because ARC is disabled for several reasons
 */

template <typename T>
class TRetainedObjCInstance
{
    static_assert(std::is_pointer_v<T>, "T must be an Objective-C pointer type");
public:
    /**
	 * Default constructor, initializes to nil.
	 */
    TRetainedObjCInstance()
    :Ptr(nil)
    {
    }

    /**
	 * Constructor, initializes instance and increases reference count if not nil.
	 */
    TRetainedObjCInstance(T InPtr)
    :Ptr(InPtr?[InPtr retain]:nil)
    {
    }

    /**
	 * Copy constructor, initializes instance and increases reference count if not nil.
     * 	
     * @param Other The Other object to copy from.
	 */
    TRetainedObjCInstance(const TRetainedObjCInstance& Other)
    :TRetainedObjCInstance(Other.Ptr)
    {
    }
    
    /**
	 * Move constructor, initializes instance. Sets other to nil without updating reference counting.
     * 	
     * @param Other The Other object to move from.
	 */
    TRetainedObjCInstance(TRetainedObjCInstance&& Other)
    :Ptr(Other.Ptr)
    {
        Other.Ptr = nil;
    }
    
    /**
	 * Copy assignment, releases owned instance and retains received one.
     * 	
     * @param Other The Other object to copy from.
	 */
    TRetainedObjCInstance& operator=(const TRetainedObjCInstance& Other)
    {
        if(this != &Other)
        {
            [Ptr release];
            Ptr = [Other.Ptr retain];
        }
        return *this;
    }
    
    /**
	 * Move assignment, releases owned instance, updates owned instance and clears Other.
     * 	
     * @param Other The Other object to move from.
	 */
    TRetainedObjCInstance operator=(TRetainedObjCInstance&& Other)
    {
        if(this != &Other)
        {
            [Ptr release];
            Ptr = Other.Ptr;
            Other.Ptr = nil;
        }
        return *this;
    }

    /**
	 * Destructor, releases owned instance if needed
	 */
    ~TRetainedObjCInstance()
    {
        Reset();
    }
    
    /**
	 * Conversion operator to underlying instance. Useful to invoke Objective-C methods transparently
	 */    
    operator T() { return Ptr; }

    /**
	 * Conversion operator to underlying instance. Useful to invoke Objective-C methods transparently
	 */    
    operator T() const { return Ptr; }

    /**
	 * Releases ownership on Objective-C instance
	 */    
    void Reset()
    {
        if(Ptr)
        {
            [Ptr release];
            Ptr = nil;
        }
    }
    
    friend bool operator==(const TRetainedObjCInstance<T>& Lhs, const TRetainedObjCInstance<T>& Rhs)
    {
        return Lhs.Ptr == Rhs.Ptr;
    }

    friend bool operator==(const TRetainedObjCInstance<T>& Lhs, T Rhs)
    {
        return Lhs.Ptr == Rhs;
    }

    friend bool operator==(T Lhs,const TRetainedObjCInstance<T>& Rhs)
    {
        return (Rhs == Lhs);
    }

    friend bool operator!=(const TRetainedObjCInstance<T>& Lhs, const TRetainedObjCInstance<T>& Rhs)
    {
        return Lhs.Ptr != Rhs.Ptr;
    }

    friend bool operator!=(const TRetainedObjCInstance<T>& Lhs, T Rhs)
    {
        return Lhs.Ptr != Rhs;
    }

    friend bool operator!=(T Lhs, const TRetainedObjCInstance<T>& Rhs)
    {
        return (Rhs != Lhs);
    }
private:
    /* Owned Objective-C instance */
    T Ptr;
};
